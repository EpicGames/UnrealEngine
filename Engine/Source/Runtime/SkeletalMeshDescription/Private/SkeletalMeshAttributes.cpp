// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshAttributes.h"
#include "MeshDescription.h"

namespace MeshAttribute
{
	namespace Vertex
	{
		const FName SkinWeights("SkinWeights");
		const FName ImportPointIndex("ImportPointIndex");
	}

	namespace PolygonGroup
	{
		const FName ParentPolygonGroup("ParentPolygonGroup");
	}

	namespace Bone
	{
		const FName Name("Name");
		const FName ParentIndex("ParentIndex");
		const FName Pose("Pose");
		const FName Color("Color");
	}

	namespace SourceGeometryPart
	{
		const FName Name("Name");
		const FName VertexOffsetAndCount("VertexOffsetAndCount");
	}
}


FName FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName("Default");

FName FSkeletalMeshAttributesShared::BonesElementName("BonesElementName");
FName FSkeletalMeshAttributesShared::SourceGeometryPartElementName("SourceGeometryPartElementName");

static FString MorphTargetAttributeNamePrefix("Morph-");


static FString SkinWeightAttributeNamePrefix()
{
	return MeshAttribute::Vertex::SkinWeights.ToString() + TEXT("-");
}


//
// FSkeletalMeshAttributes
//

FSkeletalMeshAttributes::FSkeletalMeshAttributes(FMeshDescription& InMeshDescription) :
	FStaticMeshAttributes(InMeshDescription),
	FSkeletalMeshAttributesShared(InMeshDescription)
{
	if (MeshDescription.GetElements().Contains(BonesElementName))
	{
		BoneElements = MeshDescription.GetElements()[BonesElementName].Get();
	}
	if (MeshDescription.GetElements().Contains(SourceGeometryPartElementName))
	{
		SourceGeometryPartElements = MeshDescription.GetElements()[SourceGeometryPartElementName].Get();
	}
}

void FSkeletalMeshAttributes::Register(bool bKeepExistingAttribute)
{
	if (!MeshDescription.VertexAttributes().HasAttribute(MeshAttribute::Vertex::SkinWeights) || !bKeepExistingAttribute)
	{
		MeshDescription.VertexAttributes().RegisterAttribute<int32[]>(MeshAttribute::Vertex::SkinWeights, 1, 0, EMeshAttributeFlags::Mandatory);
	}

	if (MeshDescription.GetElements().Contains(BonesElementName) == false)
	{
		BoneElementsShared = BoneElements = MeshDescription.GetElements().Emplace(BonesElementName).Get();
	}

	if (!BoneAttributes().HasAttribute(MeshAttribute::Bone::Name) || !bKeepExistingAttribute)
	{
		BoneAttributes().RegisterAttribute<FName>(MeshAttribute::Bone::Name, 1, NAME_None, EMeshAttributeFlags::Mandatory);
	}
	
	if (!BoneAttributes().HasAttribute(MeshAttribute::Bone::ParentIndex) || !bKeepExistingAttribute)
	{
		BoneAttributes().RegisterAttribute<int32>(MeshAttribute::Bone::ParentIndex, 1, INDEX_NONE, EMeshAttributeFlags::Mandatory);
	}

	if (!BoneAttributes().HasAttribute(MeshAttribute::Bone::Pose) || !bKeepExistingAttribute)
	{
		BoneAttributes().RegisterAttribute<FTransform>(MeshAttribute::Bone::Pose, 1, FTransform::Identity, EMeshAttributeFlags::Mandatory);
	}

	// Call super class
	FStaticMeshAttributes::Register(bKeepExistingAttribute);
}

void FSkeletalMeshAttributes::RegisterColorAttribute()
{
	checkSlow(MeshDescription.GetElements().Contains(BonesElementName));
	BoneAttributes().RegisterAttribute<FVector4f>(MeshAttribute::Bone::Color, 1, FVector4f(1.0f, 1.0f, 1.0f, 1.0f), EMeshAttributeFlags::Mandatory);
}


bool FSkeletalMeshAttributes::RegisterImportPointIndexAttribute()
{
	return MeshDescription.VertexAttributes().RegisterAttribute<int32>(MeshAttribute::Vertex::ImportPointIndex, 1, INDEX_NONE).IsValid();	
}


void FSkeletalMeshAttributes::UnregisterImportPointIndexAttribute()
{
	MeshDescription.VertexAttributes().UnregisterAttribute(MeshAttribute::Vertex::ImportPointIndex);	
}

bool FSkeletalMeshAttributes::RegisterPolygonGroupParentAttribute()
{
	return MeshDescription.PolygonGroupAttributes().RegisterAttribute<int32>(MeshAttribute::PolygonGroup::ParentPolygonGroup, 1, INDEX_NONE).IsValid();	
}

bool FSkeletalMeshAttributes::RegisterSkinWeightAttribute(const FName InProfileName)
{
	if (!IsValidSkinWeightProfileName(InProfileName))
	{
		return false;
	}	
	
	const FName AttributeName = CreateSkinWeightAttributeName(InProfileName);
	if (!ensure(AttributeName.IsValid()))
	{
		return false;
	}
	
	if (MeshDescription.VertexAttributes().HasAttribute(AttributeName))
	{
		return true;
	}

	return MeshDescription.VertexAttributes().RegisterAttribute<int32[]>(AttributeName, 1, 0, EMeshAttributeFlags::None).IsValid();
}

bool FSkeletalMeshAttributes::UnregisterSkinWeightAttribute(const FName InProfileName)
{
	if (!IsValidSkinWeightProfileName(InProfileName))
	{
		return false;
	}

	const FName AttributeName = CreateSkinWeightAttributeName(InProfileName);
	if (!ensure(AttributeName.IsValid()))
	{
		return false;
	}

	// Attribute not there?
	if (!MeshDescription.VertexAttributes().HasAttribute(AttributeName))
	{
		return false;
	}

	MeshDescription.VertexAttributes().UnregisterAttribute(AttributeName);
	return true;
}

FSkinWeightsVertexAttributesRef FSkeletalMeshAttributes::GetVertexSkinWeights(const FName InProfileName)
{
	return MeshDescription.VertexAttributes().GetAttributesRef<TArrayAttribute<int32>>(CreateSkinWeightAttributeName(InProfileName));
}


FSkinWeightsVertexAttributesRef FSkeletalMeshAttributes::GetVertexSkinWeightsFromAttributeName(const FName InAttributeName)
{
	if (IsSkinWeightAttribute(InAttributeName))
	{
		return MeshDescription.VertexAttributes().GetAttributesRef<TArrayAttribute<int32>>(InAttributeName);
	}
	else
	{
		return {};
	}
}


bool FSkeletalMeshAttributes::RegisterMorphTargetAttribute(
	const FName InMorphTargetName,
	const bool bIncludeNormals
	)
{
	if (InMorphTargetName.IsNone())
	{
		return false;
	}

	const FName AttributeName = CreateMorphTargetAttributeName(InMorphTargetName);
	if (!ensure(AttributeName.IsValid()))
	{
		return false;
	}

	// Register position attribute if it doesn't already exist.
	TAttributesSet<FVertexID>& VertexAttributes = MeshDescription.VertexAttributes();
	bool bSuccess = VertexAttributes.HasAttribute(AttributeName); 
	if (!bSuccess)
	{
		bSuccess = VertexAttributes.RegisterAttribute<FVector3f>(AttributeName, 1, FVector3f::ZeroVector, EMeshAttributeFlags::None).IsValid();		
	}

	TAttributesSet<FVertexInstanceID>& VertexInstanceAttributes = MeshDescription.VertexInstanceAttributes();
	if (bSuccess && bIncludeNormals)
	{
		// Register normal attribute, if requested, if it fails, then we have to unregister the vertex attribute too.
		bSuccess = VertexInstanceAttributes.RegisterAttribute<FVector3f>(AttributeName, 1, FVector3f::ZeroVector, EMeshAttributeFlags::None).IsValid();
		if (!bSuccess)
		{
			VertexAttributes.UnregisterAttribute(AttributeName);
		}
	}
	else
	{
		// Unregister normal attribute if it fails of if it's not needed anymore.
		if (VertexInstanceAttributes.HasAttribute(AttributeName))
		{
			VertexInstanceAttributes.UnregisterAttribute(AttributeName);
		}
	}

	return bSuccess;
}


bool FSkeletalMeshAttributes::UnregisterMorphTargetAttribute(const FName InMorphTargetName)
{
	if (InMorphTargetName.IsNone())
	{
		return false;
	}

	const FName AttributeName = CreateMorphTargetAttributeName(InMorphTargetName);
	if (!ensure(AttributeName.IsValid()))
	{
		return false;
	}

	// Attribute not there?
	if (!MeshDescription.VertexAttributes().HasAttribute(AttributeName))
	{
		return false;
	}

	MeshDescription.VertexAttributes().UnregisterAttribute(AttributeName);

	// VertexInstancesAttributes
	if (MeshDescription.VertexInstanceAttributes().HasAttribute(AttributeName))
	{
		MeshDescription.VertexInstanceAttributes().UnregisterAttribute(AttributeName);
	}

	return true;
}


TVertexAttributesRef<FVector3f> FSkeletalMeshAttributes::GetVertexMorphPositionDelta(const FName InMorphTargetName)
{
	return MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(CreateMorphTargetAttributeName(InMorphTargetName));
}


TVertexInstanceAttributesRef<FVector3f> FSkeletalMeshAttributes::GetVertexInstanceMorphNormalDelta(const FName InMorphTargetName)
{
	return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(CreateMorphTargetAttributeName(InMorphTargetName));
}


FSkeletalMeshAttributes::FBoneArray& FSkeletalMeshAttributes::Bones() 
{ 
	return static_cast<TMeshElementContainer<FBoneID>&>(BoneElements->Get()); 
}

TAttributesSet<FBoneID>& FSkeletalMeshAttributes::BoneAttributes() 
{ 
	return Bones().GetAttributes();
}

void FSkeletalMeshAttributes::ReserveNewBones(const int InNumBones) 
{
	BoneElements->Get().Reserve(InNumBones);
}

FBoneID FSkeletalMeshAttributes::CreateBone()
{
	return BoneElements->Get().Add();
}

void FSkeletalMeshAttributes::CreateBone(const FBoneID BoneID)
{
	BoneElements->Get().Insert(BoneID);
}

void FSkeletalMeshAttributes::DeleteBone(const FBoneID BoneID)
{
	BoneElements->Get().Remove(BoneID);
}

FSkeletalMeshAttributes::FBoneNameAttributesRef FSkeletalMeshAttributes::GetBoneNames()
{	
	return BoneAttributes().GetAttributesRef<FName>(MeshAttribute::Bone::Name);
}

FSkeletalMeshAttributes::FBoneParentIndexAttributesRef FSkeletalMeshAttributes::GetBoneParentIndices()
{	
	return BoneAttributes().GetAttributesRef<int32>(MeshAttribute::Bone::ParentIndex);
}

FSkeletalMeshAttributes::FBonePoseAttributesRef FSkeletalMeshAttributes::GetBonePoses()
{
	return BoneAttributes().GetAttributesRef<FTransform>(MeshAttribute::Bone::Pose);
}

FSkeletalMeshAttributes::FBoneColorAttributesRef FSkeletalMeshAttributes::GetBoneColors()
{
	return BoneAttributes().GetAttributesRef<FVector4f>(MeshAttribute::Bone::Color);
}

void FSkeletalMeshAttributes::RegisterSourceGeometryPartsAttributes()
{
	if (MeshDescription.GetElements().Contains(SourceGeometryPartElementName) == false)
	{
		SourceGeometryPartElementsShared = SourceGeometryPartElements = MeshDescription.GetElements().Emplace(SourceGeometryPartElementName).Get();
	}

	SourceGeometryPartAttributes().RegisterAttribute<FName>(MeshAttribute::SourceGeometryPart::Name, 1, NAME_None, EMeshAttributeFlags::Mandatory);	
	SourceGeometryPartAttributes().RegisterAttribute<int32[2]>(MeshAttribute::SourceGeometryPart::VertexOffsetAndCount, 1, {0}, EMeshAttributeFlags::Mandatory);	
}

FSkeletalMeshAttributesShared::FSourceGeometryPartArray& FSkeletalMeshAttributes::SourceGeometryParts()
{
	if (!ensure(HasSourceGeometryParts()))
	{
		static FSourceGeometryPartArray Empty;
		return Empty;
	}
	return static_cast<FSourceGeometryPartArray&>(SourceGeometryPartElements->Get());
}

TAttributesSet<FSourceGeometryPartID>& FSkeletalMeshAttributes::SourceGeometryPartAttributes()
{
	if (!ensure(HasSourceGeometryParts()))
	{
		static TAttributesSet<FSourceGeometryPartID> Empty;
		return Empty;
	}
	return SourceGeometryParts().GetAttributes();
}

FSourceGeometryPartID FSkeletalMeshAttributes::CreateSourceGeometryPart()
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return {};
	}
	
	return SourceGeometryParts().Add();
}

void FSkeletalMeshAttributes::DeleteSourceGeometryPart(FSourceGeometryPartID InSourceGeometryPartID)
{
	if (ensure(HasSourceGeometryParts()))
	{
		return SourceGeometryParts().Remove(InSourceGeometryPartID);
	}
}

FSkeletalMeshAttributesShared::FSourceGeometryPartNameRef FSkeletalMeshAttributes::GetSourceGeometryPartNames()
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return FSourceGeometryPartNameRef{};
	}
	return SourceGeometryPartAttributes().GetAttributesRef<FName>(MeshAttribute::SourceGeometryPart::Name);
}

FSkeletalMeshAttributesShared::FSourceGeometryPartVertexOffsetAndCountRef FSkeletalMeshAttributes::GetSourceGeometryPartVertexOffsetAndCounts()
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return FSourceGeometryPartVertexOffsetAndCountRef{};
	}
	return SourceGeometryPartAttributes().GetAttributesRef<TArrayView<int32>>(MeshAttribute::SourceGeometryPart::VertexOffsetAndCount);
}


//
// FSkeletalMeshAttributesShared
//

FSkeletalMeshAttributesShared::FSkeletalMeshAttributesShared(const FMeshDescription& InMeshDescription) :
	MeshDescriptionShared(InMeshDescription)
{
	if (MeshDescriptionShared.GetElements().Contains(BonesElementName))
	{
		BoneElementsShared = MeshDescriptionShared.GetElements()[BonesElementName].Get();
	}
	if (MeshDescriptionShared.GetElements().Contains(SourceGeometryPartElementName))
	{
		SourceGeometryPartElementsShared = MeshDescriptionShared.GetElements()[SourceGeometryPartElementName].Get();
	}
}

FSkinWeightsVertexAttributesConstRef FSkeletalMeshAttributesShared::GetVertexSkinWeights(const FName InProfileName) const
{
	return MeshDescriptionShared.VertexAttributes().GetAttributesRef<TArrayAttribute<int32>>(CreateSkinWeightAttributeName(InProfileName));
}

FSkinWeightsVertexAttributesConstRef FSkeletalMeshAttributesShared::GetVertexSkinWeightsFromAttributeName(const FName InAttributeName) const
{
	if (IsSkinWeightAttribute(InAttributeName))
	{
		return MeshDescriptionShared.VertexAttributes().GetAttributesRef<TArrayAttribute<int32>>(InAttributeName);
	}
	
	return {};
}

TArray<FName> FSkeletalMeshAttributesShared::GetSkinWeightProfileNames(const bool bInUserDefinedOnly) const
{
	TArray<FName> AllAttributeNames;
	MeshDescriptionShared.VertexAttributes().GetAttributeNames(AllAttributeNames);

	TArray<FName> SkinWeightProfileNames;
	bool bHasDefault = false; 

	for (const FName AttributeName: AllAttributeNames)
	{
		if (AttributeName == MeshAttribute::Vertex::SkinWeights)
		{
			bHasDefault = true;
		}
		else if (IsSkinWeightAttribute(AttributeName))
		{
			SkinWeightProfileNames.Add(GetProfileNameFromAttribute(AttributeName));
		}
	}

	SkinWeightProfileNames.Sort([](const FName A, const FName B) -> bool { return A.FastLess(B); });
	if (bHasDefault && !bInUserDefinedOnly)
	{
		SkinWeightProfileNames.Insert(DefaultSkinWeightProfileName, 0);
	}
	
	return SkinWeightProfileNames;
}

bool FSkeletalMeshAttributesShared::IsValidSkinWeightProfileName(const FName InProfileName)
{
	return !InProfileName.IsNone() && !InProfileName.IsEqual(DefaultSkinWeightProfileName, ENameCase::IgnoreCase);
}

bool FSkeletalMeshAttributesShared::IsSkinWeightAttribute(const FName InAttributeName)
{
	return InAttributeName == MeshAttribute::Vertex::SkinWeights ||
		   InAttributeName.ToString().StartsWith(SkinWeightAttributeNamePrefix());
}

FName FSkeletalMeshAttributesShared::GetProfileNameFromAttribute(const FName InAttributeName)
{
	if (InAttributeName == MeshAttribute::Vertex::SkinWeights)
	{
		return DefaultSkinWeightProfileName;
	}
	
	const FString Prefix = SkinWeightAttributeNamePrefix();
	if (InAttributeName.ToString().StartsWith(Prefix))
	{
		return FName(InAttributeName.ToString().Mid(Prefix.Len()));
	}
	else
	{
		return NAME_None;
	}
}

FName FSkeletalMeshAttributesShared::CreateSkinWeightAttributeName(const FName InProfileName)
{
	// If it's the default profile, then return the base skin weights attribute name.
	if (InProfileName.IsNone() || InProfileName.IsEqual(FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName, ENameCase::IgnoreCase))
	{
		return MeshAttribute::Vertex::SkinWeights;
	}	

	return FName(SkinWeightAttributeNamePrefix() + InProfileName.ToString());
}

FName FSkeletalMeshAttributesShared::CreateMorphTargetAttributeName(const FName InMorphTargetName)
{
	if (ensure(!InMorphTargetName.IsNone()))
	{
		return FName(MorphTargetAttributeNamePrefix + InMorphTargetName.ToString());
	}
	
	return NAME_None;
}


TArray<FName> FSkeletalMeshAttributesShared::GetMorphTargetNames() const
{
	TArray<FName> AllAttributeNames;
	MeshDescriptionShared.VertexAttributes().GetAttributeNames(AllAttributeNames);

	TArray<FName> AllMorphTargetNames;
	for (const FName AttributeName: AllAttributeNames)
	{
		if (IsMorphTargetAttribute(AttributeName))
		{
			AllMorphTargetNames.Add(GetMorphTargetNameFromAttribute(AttributeName));
		}
	}
	return AllMorphTargetNames;
}

bool FSkeletalMeshAttributesShared::IsMorphTargetAttribute(const FName InAttributeName)
{
	return InAttributeName.ToString().StartsWith(MorphTargetAttributeNamePrefix);
}

FName FSkeletalMeshAttributesShared::GetMorphTargetNameFromAttribute(const FName InAttributeName)
{
	if (ensure(IsMorphTargetAttribute(InAttributeName)))
	{
		return FName(InAttributeName.ToString().Mid(MorphTargetAttributeNamePrefix.Len()));
	}
	return NAME_None;
}

TVertexAttributesConstRef<FVector3f> FSkeletalMeshAttributesShared::GetVertexMorphPositionDelta(const FName InMorphTargetName) const
{
	return MeshDescriptionShared.VertexAttributes().GetAttributesRef<FVector3f>(CreateMorphTargetAttributeName(InMorphTargetName));
}

TVertexInstanceAttributesConstRef<FVector3f> FSkeletalMeshAttributesShared::GetVertexInstanceMorphNormalDelta(const FName InMorphTargetName) const
{
	return MeshDescriptionShared.VertexInstanceAttributes().GetAttributesRef<FVector3f>(CreateMorphTargetAttributeName(InMorphTargetName));
}

bool FSkeletalMeshAttributesShared::HasMorphTargetPositionsAttribute(const FName InMorphTargetName) const
{
	return MeshDescriptionShared.VertexAttributes().HasAttribute(CreateMorphTargetAttributeName(InMorphTargetName));
}

bool FSkeletalMeshAttributesShared::HasMorphTargetNormalsAttribute(const FName InMorphTargetName) const
{
	return MeshDescriptionShared.VertexInstanceAttributes().HasAttribute(CreateMorphTargetAttributeName(InMorphTargetName));
}


bool FSkeletalMeshAttributesShared::HasBoneColorAttribute() const
{
	return BoneAttributes().HasAttribute(MeshAttribute::Bone::Color);
}

bool FSkeletalMeshAttributesShared::HasBoneNameAttribute() const
{
	return BoneAttributes().HasAttribute(MeshAttribute::Bone::Name);
}

bool FSkeletalMeshAttributesShared::HasBonePoseAttribute() const
{
	return BoneAttributes().HasAttribute(MeshAttribute::Bone::Pose);
}

bool FSkeletalMeshAttributesShared::HasBoneParentIndexAttribute() const
{
	return BoneAttributes().HasAttribute(MeshAttribute::Bone::ParentIndex);
}

const FSkeletalMeshAttributesShared::FBoneArray& FSkeletalMeshAttributesShared::Bones() const
{ 
	return static_cast<const TMeshElementContainer<FBoneID>&>(BoneElementsShared->Get());
}

const TAttributesSet<FBoneID>& FSkeletalMeshAttributesShared::BoneAttributes() const 
{ 
	return Bones().GetAttributes(); 
}

int32 FSkeletalMeshAttributesShared::GetNumBones() const
{
	return HasBones() ? BoneElementsShared->Get().Num() : 0;
}

bool FSkeletalMeshAttributesShared::IsBoneValid(const FBoneID BoneID) const
{
	return BoneElementsShared->Get().IsValid(BoneID.GetValue());
}

FSkeletalMeshAttributesShared::FBoneNameAttributesConstRef FSkeletalMeshAttributesShared::GetBoneNames() const
{
	return BoneAttributes().GetAttributesRef<FName>(MeshAttribute::Bone::Name);
}

FSkeletalMeshAttributesShared::FBoneParentIndexAttributesConstRef FSkeletalMeshAttributesShared::GetBoneParentIndices() const
{	
	return BoneAttributes().GetAttributesRef<int32>(MeshAttribute::Bone::ParentIndex);
}

FSkeletalMeshAttributesShared::FBonePoseAttributesConstRef FSkeletalMeshAttributesShared::GetBonePoses() const
{
	return BoneAttributes().GetAttributesRef<FTransform>(MeshAttribute::Bone::Pose);
}

FSkeletalMeshAttributesShared::FBoneColorAttributesConstRef FSkeletalMeshAttributesShared::GetBoneColors() const
{
	return BoneAttributes().GetAttributesRef<FVector4f>(MeshAttribute::Bone::Color);
}

const FSkeletalMeshAttributesShared::FSourceGeometryPartArray& FSkeletalMeshAttributesShared::SourceGeometryParts() const
{
	if (!ensure(HasSourceGeometryParts()))
	{
		static FSourceGeometryPartArray Empty;
		return Empty;
	}
	return static_cast<const FSourceGeometryPartArray&>(SourceGeometryPartElementsShared->Get());
}

const TAttributesSet<FSourceGeometryPartID>& FSkeletalMeshAttributesShared::SourceGeometryPartAttributes() const
{
	if (!ensure(HasSourceGeometryParts()))
	{
		static TAttributesSet<FSourceGeometryPartID> Empty;
		return Empty;
	}
	return SourceGeometryParts().GetAttributes();
}

int32 FSkeletalMeshAttributesShared::GetNumSourceGeometryParts() const
{
	return HasSourceGeometryParts() ? SourceGeometryPartElementsShared->Get().Num() : 0;
}

bool FSkeletalMeshAttributesShared::IsSourceGeometryPartValid(const FSourceGeometryPartID InSourceGeometryPartID) const
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return false;
	}
	return SourceGeometryParts().IsValid(InSourceGeometryPartID);
}

FSkeletalMeshAttributesShared::FSourceGeometryPartNameConstRef FSkeletalMeshAttributesShared::GetSourceGeometryPartNames() const
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return FSourceGeometryPartNameConstRef{};
	}
	return SourceGeometryPartAttributes().GetAttributesRef<FName>(MeshAttribute::SourceGeometryPart::Name);
}

FSkeletalMeshAttributesShared::FSourceGeometryPartVertexOffsetAndCountConstRef FSkeletalMeshAttributesShared::GetSourceGeometryPartVertexOffsetAndCounts() const
{
	if (!ensure(HasSourceGeometryParts()))
	{
		return FSourceGeometryPartVertexOffsetAndCountConstRef{};
	}
	return SourceGeometryPartAttributes().GetAttributesRef<TArrayView<int32>>(MeshAttribute::SourceGeometryPart::VertexOffsetAndCount);
}
