// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshDescription.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshDescription)

void USkeletalMeshDescription::RegisterAttributes()
{
	RequiredAttributes = MakeUnique<FSkeletalMeshAttributes>(GetMeshDescription());
	RequiredAttributes->Register();
}
