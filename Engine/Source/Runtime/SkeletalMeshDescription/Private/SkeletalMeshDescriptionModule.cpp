// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FSkeletalMeshDescriptionModule : public IModuleInterface
{
public:

	FSkeletalMeshDescriptionModule()
	{
	}

	virtual void StartupModule() override
	{
		// Register any modular features here
	}

	virtual void ShutdownModule() override
	{
		// Unregister any modular features here
	}

private:

};

IMPLEMENT_MODULE(FSkeletalMeshDescriptionModule, SkeletalMeshDescription);
