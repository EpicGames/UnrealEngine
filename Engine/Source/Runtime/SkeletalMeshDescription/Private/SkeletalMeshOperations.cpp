// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshOperations.h"

#include "BoneWeights.h"
#include "MeshDescriptionAdapter.h"
#include "Math/GenericOctree.h"
#include "SkeletalMeshAttributes.h"
#include "Containers/Deque.h"
#include "Containers/Queue.h"
#include "Spatial/MeshAABBTree3.h"


DEFINE_LOG_CATEGORY(LogSkeletalMeshOperations);

#define LOCTEXT_NAMESPACE "SkeletalMeshOperations"

namespace UE::Private
{
	template<typename T>
	struct FCreateAndCopyAttributeValues
	{
		FCreateAndCopyAttributeValues(
			const FMeshDescription& InSourceMesh,
			FMeshDescription& InTargetMesh,
			TArray<FName>& InTargetCustomAttributeNames,
			int32 InTargetVertexIndexOffset)
			: SourceMesh(InSourceMesh)
			, TargetMesh(InTargetMesh)
			, TargetCustomAttributeNames(InTargetCustomAttributeNames)
			, TargetVertexIndexOffset(InTargetVertexIndexOffset)
			{}

		void operator()(const FName InAttributeName, TVertexAttributesConstRef<T> InSrcAttribute)
		{
			// Ignore attributes with reserved names.
			if (FSkeletalMeshAttributes::IsReservedAttributeName(InAttributeName))
			{
				return;
			}
			TAttributesSet<FVertexID>& VertexAttributes = TargetMesh.VertexAttributes();
			const bool bAppend = TargetCustomAttributeNames.Contains(InAttributeName);
			if (!bAppend)
			{
				VertexAttributes.RegisterAttribute<T>(InAttributeName, InSrcAttribute.GetNumChannels(), InSrcAttribute.GetDefaultValue(), InSrcAttribute.GetFlags());
				TargetCustomAttributeNames.Add(InAttributeName);
			}
			//Copy the data
			TVertexAttributesRef<T> TargetVertexAttributes = VertexAttributes.GetAttributesRef<T>(InAttributeName);
			for (const FVertexID SourceVertexID : SourceMesh.Vertices().GetElementIDs())
			{
				const FVertexID TargetVertexID = FVertexID(TargetVertexIndexOffset + SourceVertexID.GetValue());
				TargetVertexAttributes.Set(TargetVertexID, InSrcAttribute.Get(SourceVertexID));
			}
		}

		// Unhandled sub-types.
		void operator()(const FName, TVertexAttributesConstRef<TArrayAttribute<T>>) { }
		void operator()(const FName, TVertexAttributesConstRef<TArrayView<T>>) { }

	private:
		const FMeshDescription& SourceMesh;
		FMeshDescription& TargetMesh;
		TArray<FName>& TargetCustomAttributeNames;
		int32 TargetVertexIndexOffset = 0;
	};

} // ns UE::Private

//Add specific skeletal mesh descriptions implementation here
void FSkeletalMeshOperations::AppendSkinWeight(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, FSkeletalMeshAppendSettings& AppendSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSkeletalMeshOperations::AppendSkinWeight");
	FSkeletalMeshConstAttributes SourceSkeletalMeshAttributes(SourceMesh);
	
	FSkeletalMeshAttributes TargetSkeletalMeshAttributes(TargetMesh);
	constexpr bool bKeepExistingAttribute = true;
	TargetSkeletalMeshAttributes.Register(bKeepExistingAttribute);
	
	FSkinWeightsVertexAttributesConstRef SourceVertexSkinWeights = SourceSkeletalMeshAttributes.GetVertexSkinWeights();
	FSkinWeightsVertexAttributesRef TargetVertexSkinWeights = TargetSkeletalMeshAttributes.GetVertexSkinWeights();

	TargetMesh.SuspendVertexIndexing();
	
	//Append Custom VertexAttribute
	if(AppendSettings.bAppendVertexAttributes)
	{
		TArray<FName> TargetCustomAttributeNames;
		TargetMesh.VertexAttributes().GetAttributeNames(TargetCustomAttributeNames);

		SourceMesh.VertexAttributes().ForEachByType<float>(UE::Private::FCreateAndCopyAttributeValues<float>(SourceMesh, TargetMesh, TargetCustomAttributeNames, AppendSettings.SourceVertexIDOffset));
		SourceMesh.VertexAttributes().ForEachByType<FVector2f>(UE::Private::FCreateAndCopyAttributeValues<FVector2f>(SourceMesh, TargetMesh, TargetCustomAttributeNames, AppendSettings.SourceVertexIDOffset));
		SourceMesh.VertexAttributes().ForEachByType<FVector3f>(UE::Private::FCreateAndCopyAttributeValues<FVector3f>(SourceMesh, TargetMesh, TargetCustomAttributeNames, AppendSettings.SourceVertexIDOffset));
		SourceMesh.VertexAttributes().ForEachByType<FVector4f>(UE::Private::FCreateAndCopyAttributeValues<FVector4f>(SourceMesh, TargetMesh, TargetCustomAttributeNames, AppendSettings.SourceVertexIDOffset));
	}

	for (const FVertexID SourceVertexID : SourceMesh.Vertices().GetElementIDs())
	{
		const FVertexID TargetVertexID = FVertexID(AppendSettings.SourceVertexIDOffset + SourceVertexID.GetValue());
		FVertexBoneWeightsConst SourceBoneWeights = SourceVertexSkinWeights.Get(SourceVertexID);
		TArray<UE::AnimationCore::FBoneWeight> TargetBoneWeights;
		const int32 InfluenceCount = SourceBoneWeights.Num();
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			const FBoneIndexType SourceBoneIndex = SourceBoneWeights[InfluenceIndex].GetBoneIndex();
			if(AppendSettings.SourceRemapBoneIndex.IsValidIndex(SourceBoneIndex))
			{
				UE::AnimationCore::FBoneWeight& TargetBoneWeight = TargetBoneWeights.AddDefaulted_GetRef();
				TargetBoneWeight.SetBoneIndex(AppendSettings.SourceRemapBoneIndex[SourceBoneIndex]);
				TargetBoneWeight.SetRawWeight(SourceBoneWeights[InfluenceIndex].GetRawWeight());
			}
		}
		TargetVertexSkinWeights.Set(TargetVertexID, TargetBoneWeights);
	}

	TargetMesh.ResumeVertexIndexing();
}


bool FSkeletalMeshOperations::CopySkinWeightAttributeFromMesh(
	const FMeshDescription& InSourceMesh,
	FMeshDescription& InTargetMesh,
	const FName InSourceProfile,
	const FName InTargetProfile,
	const TMap<int32, int32>* SourceBoneIndexToTargetBoneIndexMap
	)
{
	// This is effectively a slower and dumber version of FTransferBoneWeights.
	using namespace UE::AnimationCore;
	using namespace UE::Geometry;

	FSkeletalMeshConstAttributes SourceAttributes(InSourceMesh);
	FSkeletalMeshAttributes TargetAttributes(InTargetMesh);
	
	FSkinWeightsVertexAttributesConstRef SourceWeights = SourceAttributes.GetVertexSkinWeights(InSourceProfile);
	FSkinWeightsVertexAttributesRef TargetWeights = TargetAttributes.GetVertexSkinWeights(InTargetProfile);
	TVertexAttributesConstRef<FVector3f> TargetPositions = TargetAttributes.GetVertexPositions();

	if (!SourceWeights.IsValid() || !TargetWeights.IsValid())
	{
		return false;
	}
	
	FMeshDescriptionTriangleMeshAdapter MeshAdapter(&InSourceMesh);
	TMeshAABBTree3<FMeshDescriptionTriangleMeshAdapter> BVH(&MeshAdapter);

	auto RemapBoneWeights = [SourceBoneIndexToTargetBoneIndexMap](const FVertexBoneWeightsConst& InWeights) -> FBoneWeights
	{
		TArray<FBoneWeight, TInlineAllocator<MaxInlineBoneWeightCount>> Weights;

		if (SourceBoneIndexToTargetBoneIndexMap)
		{
			for (FBoneWeight OriginalWeight: InWeights)
			{
				if (const int32* BoneIndexPtr = SourceBoneIndexToTargetBoneIndexMap->Find(OriginalWeight.GetBoneIndex()))
				{
					FBoneWeight NewWeight(static_cast<FBoneIndexType>(*BoneIndexPtr), OriginalWeight.GetRawWeight());
					Weights.Add(NewWeight);
				}
			}

			if (Weights.IsEmpty())
			{
				const FBoneWeight RootBoneWeight(0, 1.0f);
				Weights.Add(RootBoneWeight);
			}
		}
		else
		{
			for (FBoneWeight Weight: InWeights)
			{
				Weights.Add(Weight);
			}
		}
		return FBoneWeights::Create(Weights);
	};
	
	auto InterpolateWeights = [&MeshAdapter, &SourceWeights, &RemapBoneWeights](int32 InTriangleIndex, const FVector3d& InTargetPoint) -> FBoneWeights
	{
		const FDistPoint3Triangle3d Query = TMeshQueries<FMeshDescriptionTriangleMeshAdapter>::TriangleDistance(MeshAdapter, InTriangleIndex, InTargetPoint);

		const FIndex3i TriangleVertexes = MeshAdapter.GetTriangle(InTriangleIndex);
		const FVector3f BaryCoords(VectorUtil::BarycentricCoords(Query.ClosestTrianglePoint, MeshAdapter.GetVertex(TriangleVertexes.A), MeshAdapter.GetVertex(TriangleVertexes.B), MeshAdapter.GetVertex(TriangleVertexes.C)));
		const FBoneWeights WeightsA = RemapBoneWeights(SourceWeights.Get(TriangleVertexes.A));
		const FBoneWeights WeightsB = RemapBoneWeights(SourceWeights.Get(TriangleVertexes.B));
		const FBoneWeights WeightsC = RemapBoneWeights(SourceWeights.Get(TriangleVertexes.C));

		FBoneWeights BoneWeights = FBoneWeights::Blend(WeightsA, WeightsB, WeightsC, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
		
		// Blending can leave us with zero weights. Let's strip them out here.
		BoneWeights.Renormalize();
		return BoneWeights;
	};

	TArray<FBoneWeights> TargetBoneWeights;
	TargetBoneWeights.SetNum(InTargetMesh.Vertices().GetArraySize());

	ParallelFor(InTargetMesh.Vertices().GetArraySize(), [&BVH, &InTargetMesh, &TargetPositions, &TargetBoneWeights, &InterpolateWeights](int32 InVertexIndex)
	{
		const FVertexID VertexID(InVertexIndex);
		if (!InTargetMesh.Vertices().IsValid(VertexID))
		{
			return;
		}

		const FVector3d TargetPoint(TargetPositions.Get(VertexID));

		const IMeshSpatial::FQueryOptions Options;
		double NearestDistanceSquared;
		const int32 NearestTriangleIndex = BVH.FindNearestTriangle(TargetPoint, NearestDistanceSquared, Options);

		if (!ensure(NearestTriangleIndex != IndexConstants::InvalidID))
		{
			return;
		}

		TargetBoneWeights[InVertexIndex] = InterpolateWeights(NearestTriangleIndex, TargetPoint);
	});

	// Transfer the computed bone weights to the target mesh.
	for (FVertexID TargetVertexID: InTargetMesh.Vertices().GetElementIDs())
	{
		FBoneWeights& BoneWeights = TargetBoneWeights[TargetVertexID];
		if (BoneWeights.Num() == 0)
		{
			// Bind to root so that we have something.
			BoneWeights.SetBoneWeight(FBoneIndexType{0}, 1.0);
		}

		TargetWeights.Set(TargetVertexID, BoneWeights);
	}

	return true;
}

bool FSkeletalMeshOperations::RemapBoneIndicesOnSkinWeightAttribute(FMeshDescription& InMesh, TConstArrayView<int32> InBoneIndexMapping)
{
	using namespace UE::AnimationCore;
	
	FSkeletalMeshAttributes MeshAttributes(InMesh);

	// Don't renormalize, since we are not changing the weights or order.
	FBoneWeightsSettings Settings;
	Settings.SetNormalizeType(EBoneWeightNormalizeType::None);
	
	TArray<FBoneWeight> NewBoneWeights;
	for (const FName AttributeName: MeshAttributes.GetSkinWeightProfileNames())
	{
		FSkinWeightsVertexAttributesRef SkinWeights(MeshAttributes.GetVertexSkinWeights(AttributeName));

		for (FVertexID VertexID: InMesh.Vertices().GetElementIDs())
		{
			FVertexBoneWeights OldBoneWeights = SkinWeights.Get(VertexID);
			NewBoneWeights.Reset(OldBoneWeights.Num());

			for (FBoneWeight BoneWeight: OldBoneWeights)
			{
				if (!ensure(InBoneIndexMapping.IsValidIndex(BoneWeight.GetBoneIndex())))
				{
					return false;
				}
				
				BoneWeight.SetBoneIndex(InBoneIndexMapping[BoneWeight.GetBoneIndex()]);
				NewBoneWeights.Add(BoneWeight);
			}

			SkinWeights.Set(VertexID, FBoneWeights::Create(NewBoneWeights, Settings));
		}
	}
	return true;
}

namespace UE::Impl
{
static void PoseMesh(
	FMeshDescription& InOutTargetMesh,
	TConstArrayView<FMatrix44f> InRefToUserTransforms, 
	const FName InSkinWeightProfile, 
	const TMap<FName, float>& InMorphTargetWeights,
	bool bInSkipRecomputeNormalsTangents = false
	)
{
	struct FMorphInfo
	{
		TVertexAttributesRef<FVector3f> PositionDelta;
		TVertexInstanceAttributesRef<FVector3f> NormalDelta;
		float Weight = 0.0f;
	};
	
	FSkeletalMeshAttributes Attributes(InOutTargetMesh);

	// We need the mesh to be compact for the parallel for to work.
	if (InOutTargetMesh.NeedsCompact())
	{
		FElementIDRemappings Remappings;
		InOutTargetMesh.Compact(Remappings);
	}
	else
	{
		// Make sure indexers are built before entering parallel work
		InOutTargetMesh.BuildVertexIndexers();
	}

	TVertexAttributesRef<FVector3f> PositionAttribute = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> NormalAttribute = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> TangentAttribute = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> BinormalSignsAttribute = Attributes.GetVertexInstanceBinormalSigns();

	// See which morph target attributes we can peel out. If the normal attributes are not all valid, then
	// we have to automatically compute the normal from the positions. Otherwise, we only use the normal deltas. 
	TArray<FMorphInfo> MorphInfos;
	bool bAllMorphNormalsValid = true;
	for (const TPair<FName, float>& Item: InMorphTargetWeights)
	{
		const FName MorphName{Item.Key};
		const float MorphWeight{Item.Value};
		TVertexAttributesRef<FVector3f> PositionDelta = Attributes.GetVertexMorphPositionDelta(MorphName);
		// Q: Should we use the value of `r.MorphTarget.WeightThreshold` instead? The following condition is
		// identical to the default setting of that value.
		if (PositionDelta.IsValid() && !FMath::IsNearlyZero(MorphWeight))
		{
			TVertexInstanceAttributesRef<FVector3f> NormalDelta = Attributes.GetVertexInstanceMorphNormalDelta(MorphName);
			if (!NormalDelta.IsValid())
			{
				bAllMorphNormalsValid = false;
			}
			FMorphInfo MorphInfo;
			MorphInfo.PositionDelta = PositionDelta;
			MorphInfo.NormalDelta = NormalDelta;
			MorphInfo.Weight = MorphWeight;
			MorphInfos.Add(MorphInfo);
		}
	}

	// First we apply the morph info on the positions and normals.
	if (!MorphInfos.IsEmpty())
	{
		struct FMorphProcessContext
		{
			TSet<FVertexInstanceID> DirtyVertexInstances;
			TArray<FVertexID> Neighbors;
		};
		TArray<FMorphProcessContext> Contexts;
		ParallelForWithTaskContext(Contexts, InOutTargetMesh.Vertices().Num(),
			[&Mesh=InOutTargetMesh, &PositionAttribute, &MorphInfos, bAllMorphNormalsValid, bInSkipRecomputeNormalsTangents](FMorphProcessContext& Context, int32 Index)
			{
				const FVertexID VertexID{Index};

				FVector3f Position = PositionAttribute.Get(VertexID);
				bool bMoved = false;
				for (const FMorphInfo& MorphInfo: MorphInfos)
				{
					const FVector3f PositionDelta = MorphInfo.PositionDelta.Get(VertexID) * MorphInfo.Weight;
					if (!PositionDelta.IsNearlyZero())
					{
						Position += PositionDelta;
						bMoved = true;
					}
				}

				// If we need to re-generate the normals, store which vertices got moved _and_ their neighbors, since the whole
				// triangle moved, which affects neighboring vertices of the moved vertex.
				if (bMoved)
				{
					PositionAttribute.Set(VertexID, Position);
					
					if (!bAllMorphNormalsValid && !bInSkipRecomputeNormalsTangents)
					{
						Context.DirtyVertexInstances.Append(Mesh.GetVertexVertexInstanceIDs(VertexID));

						Mesh.GetVertexAdjacentVertices(VertexID, Context.Neighbors);
						for (const FVertexID NeighborVertexID: Context.Neighbors)
						{
							Context.DirtyVertexInstances.Append(Mesh.GetVertexVertexInstanceIDs(NeighborVertexID));
						}
					}
				}
			});

		if (bAllMorphNormalsValid)
		{
			ParallelForWithTaskContext(Contexts, InOutTargetMesh.VertexInstances().Num(),
				[&MorphInfos, &NormalAttribute, &TangentAttribute, &BinormalSignsAttribute, bInSkipRecomputeNormalsTangents](FMorphProcessContext& Context, int32 Index)
				{
					FVertexInstanceID VertexInstanceID{Index};

					FVector3f Normal = NormalAttribute.Get(VertexInstanceID);
					FVector3f Tangent = TangentAttribute.Get(VertexInstanceID);
					FVector3f Binormal = FVector3f::CrossProduct(Normal, Tangent) * BinormalSignsAttribute.Get(VertexInstanceID);
					
					bool bMoved = false;
					for (const FMorphInfo& MorphInfo: MorphInfos)
					{
						const FVector3f NormalDelta = MorphInfo.NormalDelta.Get(VertexInstanceID) * MorphInfo.Weight;
						if (!NormalDelta.IsNearlyZero())
						{
							Normal += NormalDelta;
							bMoved = true;
						}
					}

					if (bMoved)
					{
						if (Normal.Normalize())
						{
							// Badly named function. This orthonormalizes X & Y using Z as the control. 
							FVector3f::CreateOrthonormalBasis(Tangent, Binormal, Normal);
							
							NormalAttribute.Set(VertexInstanceID, Normal);
							TangentAttribute.Set(VertexInstanceID, Tangent);
							const float BinormalSign = FMatrix44f(Tangent, Binormal, Normal, FVector3f::ZeroVector).Determinant() < 0 ? -1.0f : +1.0f;
							BinormalSignsAttribute.Set(VertexInstanceID, BinormalSign);
						}
						else
						{
							if (!bInSkipRecomputeNormalsTangents)
							{
								// Something went wrong. Reconstruct the normal from the tangent and binormal.
								Context.DirtyVertexInstances.Add(VertexInstanceID);
							}
						}
					}
				});
		}

		if (!bInSkipRecomputeNormalsTangents)
		{
			// Clear out any normals that were affected by the point move, or ended up being degenerate during normal offsetting.
			TSet<FVertexInstanceID> DirtyVertexInstances;
			for (const FMorphProcessContext& ProcessContext: Contexts)
			{
				DirtyVertexInstances.Append(ProcessContext.DirtyVertexInstances);
			}
		
			if (!DirtyVertexInstances.IsEmpty())
			{
				// Mark any vector as zero that we want to regenerate from triangle + neighbors + tangents.
				for (const FVertexInstanceID VertexInstanceID: DirtyVertexInstances)
				{
					NormalAttribute.Set(VertexInstanceID, FVector3f::ZeroVector);
				}

				FSkeletalMeshOperations::ComputeTriangleTangentsAndNormals(InOutTargetMesh, UE_SMALL_NUMBER, nullptr);

				// Compute the normals on the dirty vertices, and adjust the tangents to match.
				FSkeletalMeshOperations::ComputeTangentsAndNormals(InOutTargetMesh, EComputeNTBsFlags::WeightedNTBs);
			
				// We don't need the triangle tangents and normals anymore.
				Attributes.UnregisterTriangleNormalAndTangentAttributes();
			}	
		}
	}

	using namespace UE::AnimationCore;

	// The normal needs to be transformed using the inverse transpose of the transform matrices to ensure that
	// scaling works correctly. 
	TArray<FMatrix44f> RefToUserTransformsNormal;
	RefToUserTransformsNormal.Reserve(InRefToUserTransforms.Num());
	for (const FMatrix44f& Mat: InRefToUserTransforms)
	{
		RefToUserTransformsNormal.Add(Mat.Inverse().GetTransposed());
	}
	
	FSkinWeightsVertexAttributesRef SkinWeightAttribute = Attributes.GetVertexSkinWeights(InSkinWeightProfile);
	ParallelFor(InOutTargetMesh.Vertices().Num(),
		[&Mesh=InOutTargetMesh, &PositionAttribute, &NormalAttribute, &TangentAttribute, &SkinWeightAttribute, &RefToUserTransforms=InRefToUserTransforms, &RefToUserTransformsNormal](int32 Index)
		{
			const FVertexID VertexID(Index);
			const FVertexBoneWeights BoneWeights = SkinWeightAttribute.Get(VertexID);
			const FVector3f Position = PositionAttribute.Get(VertexID);
			FVector3f SkinnedPosition = FVector3f::ZeroVector;

			for (const FBoneWeight BW: BoneWeights)
			{
				SkinnedPosition += RefToUserTransforms[BW.GetBoneIndex()].TransformPosition(Position) * BW.GetWeight();
			}
			PositionAttribute.Set(VertexID, SkinnedPosition);
			
			for (const FVertexInstanceID VertexInstanceID: Mesh.GetVertexVertexInstanceIDs(VertexID))
			{
				const FVector3f Normal = NormalAttribute.Get(VertexInstanceID);
				const FVector3f Tangent = TangentAttribute.Get(VertexInstanceID);
				FVector3f SkinnedNormal = FVector3f::ZeroVector;
				FVector3f SkinnedTangent = FVector3f::ZeroVector;

				for (const FBoneWeight BW: BoneWeights)
				{
					SkinnedNormal += RefToUserTransformsNormal[BW.GetBoneIndex()].TransformVector(Normal) * BW.GetWeight();
					SkinnedTangent += RefToUserTransforms[BW.GetBoneIndex()].TransformVector(Tangent) * BW.GetWeight();
				}
				
				SkinnedNormal.Normalize();
				SkinnedTangent.Normalize();

				NormalAttribute.Set(VertexInstanceID, SkinnedNormal);
				TangentAttribute.Set(VertexInstanceID, SkinnedTangent);
			}
		});
}

static void UnposeMesh(
	FMeshDescription& InOutTargetMesh,
	const FMeshDescription& InRefMesh,
	TConstArrayView<FMatrix44f> InRefToUserTransforms,
	const FName InSkinWeightProfile,
	const TMap<FName, float>& InMorphTargetWeights
)
{
	struct FMorphInfo
	{
		TVertexAttributesConstRef<FVector3f> PositionDelta;
		TVertexInstanceAttributesConstRef<FVector3f> NormalDelta;
		float Weight = 0.0f;
	};

	FSkeletalMeshAttributes Attributes(InOutTargetMesh);
	FSkeletalMeshConstAttributes RefAttributes(InRefMesh);

	// We need the mesh to be compact for the parallel for to work.
	if (InOutTargetMesh.NeedsCompact())
	{
		FElementIDRemappings Remappings;
		InOutTargetMesh.Compact(Remappings);
	}
	else
	{
		// Make sure indexers are built before entering parallel work
		InOutTargetMesh.BuildVertexIndexers();
	}

	TVertexAttributesRef<FVector3f> PositionAttribute = Attributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> NormalAttribute = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> TangentAttribute = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> BinormalSignsAttribute = Attributes.GetVertexInstanceBinormalSigns();


	// Invert skinning first

	using namespace UE::AnimationCore;

	// The normal needs to be transformed using the inverse transpose of the transform matrices to ensure that
	// scaling works correctly. 
	TArray<FMatrix44f> RefToUserTransformsNormal;
	RefToUserTransformsNormal.Reserve(InRefToUserTransforms.Num());
	for (const FMatrix44f& Mat : InRefToUserTransforms)
	{
		RefToUserTransformsNormal.Add(Mat.Inverse().GetTransposed());
	}

	FSkinWeightsVertexAttributesRef SkinWeightAttribute = Attributes.GetVertexSkinWeights(InSkinWeightProfile);
	ParallelFor(InOutTargetMesh.Vertices().Num(),
		[&Mesh = InOutTargetMesh, &PositionAttribute, &NormalAttribute, &TangentAttribute, &SkinWeightAttribute, &RefToUserTransforms = InRefToUserTransforms, &RefToUserTransformsNormal](int32 Index)
		{
			const FVertexID VertexID(Index);
			const FVertexBoneWeights BoneWeights = SkinWeightAttribute.Get(VertexID);
			const FVector3f Position = PositionAttribute.Get(VertexID);

			FMatrix44f SkinMatrix = FMatrix44f(FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector);
			SkinMatrix.M[3][3] = 0.0f;

			for (const FBoneWeight BW : BoneWeights)
			{
				SkinMatrix += RefToUserTransforms[BW.GetBoneIndex()] * BW.GetWeight();
			}
			FVector3f SkinnedPosition = SkinMatrix.Inverse().TransformPosition(Position);
			PositionAttribute.Set(VertexID, SkinnedPosition);

			for (const FVertexInstanceID VertexInstanceID : Mesh.GetVertexVertexInstanceIDs(VertexID))
			{
				const FVector3f Normal = NormalAttribute.Get(VertexInstanceID);
				const FVector3f Tangent = TangentAttribute.Get(VertexInstanceID);
				FVector3f SkinnedNormal = FVector3f::ZeroVector;
				FVector3f SkinnedTangent = FVector3f::ZeroVector;

				for (const FBoneWeight BW : BoneWeights)
				{
					SkinnedNormal += RefToUserTransformsNormal[BW.GetBoneIndex()].TransformVector(Normal) * BW.GetWeight();
					SkinnedTangent += RefToUserTransforms[BW.GetBoneIndex()].TransformVector(Tangent) * BW.GetWeight();
				}

				SkinnedNormal.Normalize();
				SkinnedTangent.Normalize();

				NormalAttribute.Set(VertexInstanceID, SkinnedNormal);
				TangentAttribute.Set(VertexInstanceID, SkinnedTangent);
			}
		});


	// See which morph target attributes we can peel out. If the normal attributes are not all valid, then
	// we have to automatically compute the normal from the positions. Otherwise, we only use the normal deltas. 
	TArray<FMorphInfo> MorphInfos;
	bool bAllMorphNormalsValid = true;
	for (const TPair<FName, float>& Item : InMorphTargetWeights)
	{
		const FName MorphName{ Item.Key };
		const float MorphWeight{ Item.Value };
		TVertexAttributesConstRef<FVector3f> PositionDelta = RefAttributes.GetVertexMorphPositionDelta(MorphName);
		// Q: Should we use the value of `r.MorphTarget.WeightThreshold` instead? The following condition is
		// identical to the default setting of that value.
		if (PositionDelta.IsValid() && !FMath::IsNearlyZero(MorphWeight))
		{
			TVertexInstanceAttributesConstRef<FVector3f> NormalDelta = RefAttributes.GetVertexInstanceMorphNormalDelta(MorphName);
			if (!NormalDelta.IsValid())
			{
				bAllMorphNormalsValid = false;
			}
			FMorphInfo MorphInfo;
			MorphInfo.PositionDelta = PositionDelta;
			MorphInfo.NormalDelta = NormalDelta;
			MorphInfo.Weight = MorphWeight;
			MorphInfos.Add(MorphInfo);
		}
	}

	// Inverse morph deltas

	if (!MorphInfos.IsEmpty())
	{
		struct FMorphProcessContext
		{
			TSet<FVertexInstanceID> DirtyVertexInstances;
			TArray<FVertexID> Neighbors;
		};
		TArray<FMorphProcessContext> Contexts;
		ParallelForWithTaskContext(Contexts, InOutTargetMesh.Vertices().Num(),
			[&Mesh = InOutTargetMesh, &PositionAttribute, &MorphInfos, bAllMorphNormalsValid](FMorphProcessContext& Context, int32 Index)
			{
				const FVertexID VertexID{ Index };

				FVector3f Position = PositionAttribute.Get(VertexID);
				bool bMoved = false;
				for (const FMorphInfo& MorphInfo : MorphInfos)
				{
					const FVector3f PositionDelta = MorphInfo.PositionDelta.Get(VertexID) * MorphInfo.Weight * -1.0;
					if (!PositionDelta.IsNearlyZero())
					{
						Position += PositionDelta;
						bMoved = true;
					}
				}

				// If we need to re-generate the normals, store which vertices got moved _and_ their neighbors, since the whole
				// triangle moved, which affects neighboring vertices of the moved vertex.
				if (bMoved)
				{
					PositionAttribute.Set(VertexID, Position);

					if (!bAllMorphNormalsValid)
					{
						Context.DirtyVertexInstances.Append(Mesh.GetVertexVertexInstanceIDs(VertexID));

						Mesh.GetVertexAdjacentVertices(VertexID, Context.Neighbors);
						for (const FVertexID NeighborVertexID : Context.Neighbors)
						{
							Context.DirtyVertexInstances.Append(Mesh.GetVertexVertexInstanceIDs(NeighborVertexID));
						}
					}
				}
			});

		if (bAllMorphNormalsValid)
		{
			ParallelForWithTaskContext(Contexts, InOutTargetMesh.VertexInstances().Num(),
				[&MorphInfos, &NormalAttribute, &TangentAttribute, &BinormalSignsAttribute](FMorphProcessContext& Context, int32 Index)
				{
					FVertexInstanceID VertexInstanceID{Index};

					FVector3f Normal = NormalAttribute.Get(VertexInstanceID);
					FVector3f Tangent = TangentAttribute.Get(VertexInstanceID);
					FVector3f Binormal = FVector3f::CrossProduct(Normal, Tangent) * BinormalSignsAttribute.Get(VertexInstanceID);
					
					bool bMoved = false;
					for (const FMorphInfo& MorphInfo: MorphInfos)
					{
						const FVector3f NormalDelta = MorphInfo.NormalDelta.Get(VertexInstanceID) * MorphInfo.Weight * -1.0f;
						if (!NormalDelta.IsNearlyZero())
						{
							Normal += NormalDelta;
							bMoved = true;
						}
					}

					if (bMoved)
					{
						if (Normal.Normalize())
						{
							// Badly named function. This orthonormalizes X & Y using Z as the control. 
							FVector3f::CreateOrthonormalBasis(Tangent, Binormal, Normal);
							
							NormalAttribute.Set(VertexInstanceID, Normal);
							TangentAttribute.Set(VertexInstanceID, Tangent);
							const float BinormalSign = FMatrix44f(Tangent, Binormal, Normal, FVector3f::ZeroVector).Determinant() < 0 ? -1.0f : +1.0f;
							BinormalSignsAttribute.Set(VertexInstanceID, BinormalSign);
						}
						else
						{
							// Something went wrong. Reconstruct the normal from the tangent and binormal.
							Context.DirtyVertexInstances.Add(VertexInstanceID);
						}
					}
				});
		}

		// Clear out any normals that were affected by the point move, or ended up being degenerate during normal offsetting.
		TSet<FVertexInstanceID> DirtyVertexInstances;
		for (const FMorphProcessContext& ProcessContext: Contexts)
		{
			DirtyVertexInstances.Append(ProcessContext.DirtyVertexInstances);
		}
		
		if (!DirtyVertexInstances.IsEmpty())
		{
			// Mark any vector as zero that we want to regenerate from triangle + neighbors + tangents.
			for (const FVertexInstanceID VertexInstanceID: DirtyVertexInstances)
			{
				NormalAttribute.Set(VertexInstanceID, FVector3f::ZeroVector);
			}

			FSkeletalMeshOperations::ComputeTriangleTangentsAndNormals(InOutTargetMesh, UE_SMALL_NUMBER, nullptr);

			// Compute the normals on the dirty vertices, and adjust the tangents to match.
			FSkeletalMeshOperations::ComputeTangentsAndNormals(InOutTargetMesh, EComputeNTBsFlags::WeightedNTBs);
			
			// We don't need the triangle tangents and normals anymore.
			InOutTargetMesh.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Normal);
			InOutTargetMesh.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Tangent);
			InOutTargetMesh.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Binormal);
		}	
	}
}
	
} // namespace Impl

bool FSkeletalMeshOperations::GetPosedMesh(
	const FMeshDescription& InSourceMesh, 
	FMeshDescription& OutTargetMesh,
	TConstArrayView<FTransform> InComponentSpaceTransforms, 
	const FName InSkinWeightProfile, 
	const TMap<FName, float>& InMorphTargetWeights
	)
{
	// Verify that the mesh is valid.
	FSkeletalMeshConstAttributes Attributes(InSourceMesh);
	if (!Attributes.HasBonePoseAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return false;
	}

	if (!Attributes.GetVertexSkinWeights(InSkinWeightProfile).IsValid())
	{
		return false;
	}

	if (Attributes.GetNumBones() != InComponentSpaceTransforms.Num())
	{
		return false;
	}

	// Convert the component-space transforms into a set of matrices that transform from the reference pose to
	// the user pose. These are then used to nudge the vertices from the reference pose to the wanted
	// user pose by weighing the influence of each bone on a given vertex. If the user pose and the reference pose
	// are identical, these are all identity matrices.
	TArray<FMatrix44f> RefToUserTransforms;
	TArray<FMatrix44f> RefPoseTransforms;
	
	FSkeletalMeshAttributes::FBonePoseAttributesConstRef BonePoseAttribute = Attributes.GetBonePoses(); 
	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef ParentBoneIndexAttribute = Attributes.GetBoneParentIndices(); 
	RefToUserTransforms.SetNumUninitialized(Attributes.GetNumBones());
	RefPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());
	
	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
		RefPoseTransforms[BoneIndex] = FMatrix44f{BonePoseAttribute.Get(BoneIndex).ToMatrixWithScale()};

		if (ParentBoneIndex != INDEX_NONE)
		{
			RefPoseTransforms[BoneIndex] = RefPoseTransforms[BoneIndex] * RefPoseTransforms[ParentBoneIndex];
		}

		RefToUserTransforms[BoneIndex] = RefPoseTransforms[BoneIndex].Inverse() * FMatrix44f{InComponentSpaceTransforms[BoneIndex].ToMatrixWithScale()};
	}
	
	// Start with a fresh duplicate and then pose the target mesh in-place.
	OutTargetMesh = InSourceMesh;
	UE::Impl::PoseMesh(OutTargetMesh, RefToUserTransforms, InSkinWeightProfile, InMorphTargetWeights);

	// Write out the current ref pose (in bone-space) to the mesh. 
	FSkeletalMeshAttributes WriteAttributes(OutTargetMesh);
	FSkeletalMeshAttributes::FBonePoseAttributesRef WriteBonePoseAttribute = WriteAttributes.GetBonePoses(); 
	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
		FTransform RefPoseTransform = InComponentSpaceTransforms[BoneIndex];

		if (ParentBoneIndex != INDEX_NONE)
		{
			RefPoseTransform = RefPoseTransform.GetRelativeTransform(InComponentSpaceTransforms[ParentBoneIndex]);
		}
		WriteBonePoseAttribute.Set(BoneIndex, RefPoseTransform);
	}
	
	return true;
}

bool FSkeletalMeshOperations::GetPosedMeshInPlace(FMeshDescription& InOutTargetMesh, TConstArrayView<FTransform> InComponentSpaceTransforms,
	const FName InSkinWeightProfile, const TMap<FName, float>& InMorphTargetWeights, bool bInSkipRecomputeNormalsTangents, bool bInWriteBonePose)
{
	// Verify that the mesh is valid.
	FSkeletalMeshConstAttributes Attributes(InOutTargetMesh);
	if (!Attributes.HasBonePoseAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return false;
	}

	if (!Attributes.GetVertexSkinWeights(InSkinWeightProfile).IsValid())
	{
		return false;
	}

	if (Attributes.GetNumBones() != InComponentSpaceTransforms.Num())
	{
		return false;
	}

	// Convert the component-space transforms into a set of matrices that transform from the reference pose to
	// the user pose. These are then used to nudge the vertices from the reference pose to the wanted
	// user pose by weighing the influence of each bone on a given vertex. If the user pose and the reference pose
	// are identical, these are all identity matrices.
	TArray<FMatrix44f> RefToUserTransforms;
	TArray<FMatrix44f> RefPoseTransforms;
	
	FSkeletalMeshAttributes::FBonePoseAttributesConstRef BonePoseAttribute = Attributes.GetBonePoses(); 
	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef ParentBoneIndexAttribute = Attributes.GetBoneParentIndices(); 
	RefToUserTransforms.SetNumUninitialized(Attributes.GetNumBones());
	RefPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());
	
	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
		RefPoseTransforms[BoneIndex] = FMatrix44f{BonePoseAttribute.Get(BoneIndex).ToMatrixWithScale()};

		if (ParentBoneIndex != INDEX_NONE)
		{
			RefPoseTransforms[BoneIndex] = RefPoseTransforms[BoneIndex] * RefPoseTransforms[ParentBoneIndex];
		}

		RefToUserTransforms[BoneIndex] = RefPoseTransforms[BoneIndex].Inverse() * FMatrix44f{InComponentSpaceTransforms[BoneIndex].ToMatrixWithScale()};
	}
	
	// Pose the target mesh in-place.
	UE::Impl::PoseMesh(InOutTargetMesh, RefToUserTransforms, InSkinWeightProfile, InMorphTargetWeights, bInSkipRecomputeNormalsTangents);

	if (bInWriteBonePose)
	{
		// Write out the current ref pose (in bone-space) to the mesh. 
		FSkeletalMeshAttributes WriteAttributes(InOutTargetMesh);
		FSkeletalMeshAttributes::FBonePoseAttributesRef WriteBonePoseAttribute = WriteAttributes.GetBonePoses(); 
		for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
		{
			const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
			FTransform RefPoseTransform = InComponentSpaceTransforms[BoneIndex];

			if (ParentBoneIndex != INDEX_NONE)
			{
				RefPoseTransform = RefPoseTransform.GetRelativeTransform(InComponentSpaceTransforms[ParentBoneIndex]);
			}
			WriteBonePoseAttribute.Set(BoneIndex, RefPoseTransform);
		}
	}

	
	return true;
}


bool FSkeletalMeshOperations::GetPosedMesh(
	const FMeshDescription& InSourceMesh,
	FMeshDescription& OutTargetMesh,
	const TMap<FName, FTransform>& InBoneSpaceTransforms,
	const FName InSkinWeightProfile, 
	const TMap<FName, float>& InMorphTargetWeights
	)
{
	// Verify that the mesh is valid.
	FSkeletalMeshConstAttributes Attributes(InSourceMesh);
	if (!Attributes.HasBoneNameAttribute() || !Attributes.HasBonePoseAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return false;
	}

	if (!Attributes.GetVertexSkinWeights(InSkinWeightProfile).IsValid())
	{
		return false;
	}

	TArray<FMatrix44f> RefToUserTransforms;
	TArray<FMatrix44f> RefPoseTransforms;
	TArray<FMatrix44f> UserPoseTransforms;
	
	FSkeletalMeshAttributes::FBoneNameAttributesConstRef BoneNameAttribute = Attributes.GetBoneNames(); 
	FSkeletalMeshAttributes::FBonePoseAttributesConstRef BonePoseAttribute = Attributes.GetBonePoses(); 
	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef ParentBoneIndexAttribute = Attributes.GetBoneParentIndices();
	
	RefToUserTransforms.SetNumUninitialized(Attributes.GetNumBones());
	RefPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());
	UserPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());

	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const FName BoneName = BoneNameAttribute.Get(BoneIndex);
		const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
		RefPoseTransforms[BoneIndex] = FMatrix44f{BonePoseAttribute.Get(BoneIndex).ToMatrixWithScale()};
		if (const FTransform* UserTransform = InBoneSpaceTransforms.Find(BoneName))
		{
			UserPoseTransforms[BoneIndex] = FMatrix44f{UserTransform->ToMatrixWithScale()};

			// Update the pose on the mesh to match the user pose.
		}
		else
		{
			UserPoseTransforms[BoneIndex] = RefPoseTransforms[BoneIndex];
		}
		
		if (ParentBoneIndex != INDEX_NONE)
		{
			RefPoseTransforms[BoneIndex] = RefPoseTransforms[BoneIndex] * RefPoseTransforms[ParentBoneIndex];
			UserPoseTransforms[BoneIndex] = UserPoseTransforms[BoneIndex] * UserPoseTransforms[ParentBoneIndex];
		}

		RefToUserTransforms[BoneIndex] = RefPoseTransforms[BoneIndex].Inverse() * UserPoseTransforms[BoneIndex];
	}

	// Start with a fresh duplicate and then pose the target mesh in-place.
	OutTargetMesh = InSourceMesh;
	UE::Impl::PoseMesh(OutTargetMesh, RefToUserTransforms, InSkinWeightProfile, InMorphTargetWeights);

	FSkeletalMeshAttributes WriteAttributes(OutTargetMesh);
	FSkeletalMeshAttributes::FBonePoseAttributesRef WriteBonePoseAttribute = WriteAttributes.GetBonePoses(); 
	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const FName BoneName = BoneNameAttribute.Get(BoneIndex);
		if (const FTransform* UserTransform = InBoneSpaceTransforms.Find(BoneName))
		{
			WriteBonePoseAttribute.Set(BoneIndex, *UserTransform);
		}
	}

	return true;
}

bool FSkeletalMeshOperations::GetUnposedMesh(
	const FMeshDescription& InPosedMesh,
	const FMeshDescription& InRefMesh,
	TArray<FTransform>& RefBoneTransforms,
	FMeshDescription& OutUnposedMesh,
	TConstArrayView<FTransform> InComponentSpaceTransforms,
	const FName InSkinWeightProfile,
	const TMap<FName, float>& InMorphTargetWeights
)
{
	// Verify that the mesh is valid.
	FSkeletalMeshConstAttributes Attributes(InPosedMesh);
	if (!Attributes.HasBonePoseAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return false;
	}

	if (!Attributes.GetVertexSkinWeights(InSkinWeightProfile).IsValid())
	{
		return false;
	}

	if (Attributes.GetNumBones() != InComponentSpaceTransforms.Num())
	{
		return false;
	}

	// Convert the component-space transforms into a set of matrices that transform from the reference pose to
	// the user pose. These are then used to nudge the vertices from the reference pose to the wanted
	// user pose by weighing the influence of each bone on a given vertex. If the user pose and the reference pose
	// are identical, these are all identity matrices.
	TArray<FMatrix44f> RefToUserTransforms;
	TArray<FMatrix44f> RefPoseTransforms;

	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef ParentBoneIndexAttribute = Attributes.GetBoneParentIndices();
	RefToUserTransforms.SetNumUninitialized(Attributes.GetNumBones());
	RefPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());

	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		FTransform ReferenceBoneTransform = RefBoneTransforms[BoneIndex];
		RefPoseTransforms[BoneIndex] = FMatrix44f{ ReferenceBoneTransform.ToMatrixWithScale() };
		RefToUserTransforms[BoneIndex] = RefPoseTransforms[BoneIndex].Inverse() * FMatrix44f { InComponentSpaceTransforms[BoneIndex].ToMatrixWithScale() };
	}

	// Start with a fresh duplicate and then pose the target mesh in-place.
	OutUnposedMesh = InPosedMesh;
	UE::Impl::UnposeMesh(OutUnposedMesh, InRefMesh, RefToUserTransforms, InSkinWeightProfile, InMorphTargetWeights);

	// Write out the current ref pose (in bone-space) to the mesh. 
	FSkeletalMeshAttributes WriteAttributes(OutUnposedMesh);
	FSkeletalMeshAttributes::FBonePoseAttributesRef WriteBonePoseAttribute = WriteAttributes.GetBonePoses();
	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
		FTransform RefPoseTransform = InComponentSpaceTransforms[BoneIndex];

		if (ParentBoneIndex != INDEX_NONE)
		{
			RefPoseTransform = RefPoseTransform.GetRelativeTransform(InComponentSpaceTransforms[ParentBoneIndex]);
		}
		WriteBonePoseAttribute.Set(BoneIndex, RefPoseTransform);
	}

	return true;
}

bool FSkeletalMeshOperations::GetUnposedMeshInPlace(FMeshDescription& InOutTargetMesh, const FMeshDescription& InRefMesh,
	TArray<FTransform>& RefBoneTransforms, TConstArrayView<FTransform> InComponentSpaceTransforms, const FName InSkinWeightProfile,
	const TMap<FName, float>& InMorphTargetWeights, bool bInWriteBonePose)
{
	// Verify that the mesh is valid.
	FSkeletalMeshConstAttributes Attributes(InOutTargetMesh);
	if (!Attributes.HasBonePoseAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return false;
	}

	if (!Attributes.GetVertexSkinWeights(InSkinWeightProfile).IsValid())
	{
		return false;
	}

	if (Attributes.GetNumBones() != InComponentSpaceTransforms.Num())
	{
		return false;
	}

	// Convert the component-space transforms into a set of matrices that transform from the reference pose to
	// the user pose. These are then used to nudge the vertices from the reference pose to the wanted
	// user pose by weighing the influence of each bone on a given vertex. If the user pose and the reference pose
	// are identical, these are all identity matrices.
	TArray<FMatrix44f> RefToUserTransforms;
	TArray<FMatrix44f> RefPoseTransforms;

	FSkeletalMeshAttributes::FBoneParentIndexAttributesConstRef ParentBoneIndexAttribute = Attributes.GetBoneParentIndices();
	RefToUserTransforms.SetNumUninitialized(Attributes.GetNumBones());
	RefPoseTransforms.SetNumUninitialized(Attributes.GetNumBones());

	for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
	{
		FTransform ReferenceBoneTransform = RefBoneTransforms[BoneIndex];
		RefPoseTransforms[BoneIndex] = FMatrix44f{ ReferenceBoneTransform.ToMatrixWithScale() };
		RefToUserTransforms[BoneIndex] = RefPoseTransforms[BoneIndex].Inverse() * FMatrix44f { InComponentSpaceTransforms[BoneIndex].ToMatrixWithScale() };
	}

	UE::Impl::UnposeMesh(InOutTargetMesh, InRefMesh, RefToUserTransforms, InSkinWeightProfile, InMorphTargetWeights);

	if (bInWriteBonePose)
	{
		// Write out the current ref pose (in bone-space) to the mesh. 
		FSkeletalMeshAttributes WriteAttributes(InOutTargetMesh);
		FSkeletalMeshAttributes::FBonePoseAttributesRef WriteBonePoseAttribute = WriteAttributes.GetBonePoses();
		for (int32 BoneIndex = 0; BoneIndex < Attributes.GetNumBones(); BoneIndex++)
		{
			const int32 ParentBoneIndex = ParentBoneIndexAttribute.Get(BoneIndex);
			FTransform RefPoseTransform = InComponentSpaceTransforms[BoneIndex];

			if (ParentBoneIndex != INDEX_NONE)
			{
				RefPoseTransform = RefPoseTransform.GetRelativeTransform(InComponentSpaceTransforms[ParentBoneIndex]);
			}
			WriteBonePoseAttribute.Set(BoneIndex, RefPoseTransform);
		}	
	}

	return true;
}

void FSkeletalMeshOperations::ConvertHardEdgesToSmoothMasks(
	const FMeshDescription& InMeshDescription,
	TArray<uint32>& OutSmoothMasks
)
{
	OutSmoothMasks.SetNumZeroed(InMeshDescription.Triangles().Num());

	TSet<FTriangleID> ProcessedTriangles;
	TArray<FTriangleID> TriangleQueue;
	uint32 CurrentSmoothMask = 1;

	const TEdgeAttributesConstRef<bool> IsEdgeHard = InMeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);

	for (FTriangleID SeedTriangleID : InMeshDescription.Triangles().GetElementIDs())
	{
		if (ProcessedTriangles.Contains(SeedTriangleID))
		{
			continue;
		}

		TriangleQueue.Push(SeedTriangleID);
		while (!TriangleQueue.IsEmpty())
		{
			const FTriangleID TriangleID = TriangleQueue.Pop(EAllowShrinking::No);
			TArrayView<const FEdgeID> TriangleEdges = InMeshDescription.GetTriangleEdges(TriangleID);

			OutSmoothMasks[TriangleID.GetValue()] = CurrentSmoothMask;
			ProcessedTriangles.Add(TriangleID);

			for (const FEdgeID EdgeID : TriangleEdges)
			{
				if (!IsEdgeHard.Get(EdgeID))
				{
					TArrayView<const FTriangleID> ConnectedTriangles = InMeshDescription.GetEdgeConnectedTriangleIDs(EdgeID);
					for (const FTriangleID NeighborTriangleID : ConnectedTriangles)
					{
						if (!ProcessedTriangles.Contains(NeighborTriangleID))
						{
							TriangleQueue.Push(NeighborTriangleID);
						}
					}
				}
			}
		}

		CurrentSmoothMask <<= 1;
		if (CurrentSmoothMask == 0)
		{
			// If we exhausted all available bits, then thunk to the more complete algorithm. For reasons unknown at this time, it doesn't generate
			// nice smooth groups for some simpler test objects. For more complex input products it does a decent job though.
			OutSmoothMasks.SetNumZeroed(InMeshDescription.Triangles().Num());
			FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(InMeshDescription, OutSmoothMasks);
			break;
		}
	}
}

void FSkeletalMeshOperations::FixVertexInstanceStructure(FMeshDescription& SourceMeshDescription, FMeshDescription& TargetMeshDescription /*Expected to be empty*/, const TArray<uint32>& SourceSmoothingMasks, TArray<uint32>& TargetFaceSmoothingMasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FixVertexInstanceStructure)

	if (!ensure(TargetMeshDescription.IsEmpty()))
	{
		return;
	}

	struct FMeshAttributesHelper
	{
		FSkeletalMeshAttributes MeshAttributes;

		//For read/write
		TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames;

		//For read/write Vertex attributes:
		TVertexAttributesRef<FVector3f> Positions;
		FSkinWeightsVertexAttributesRef SkinWeights;

		//For read/write VertexInstance attributes
		TVertexInstanceAttributesRef<FVector3f> Normals;
		TVertexInstanceAttributesRef<FVector3f> Tangents;
		TVertexInstanceAttributesRef<float>		BinormalSigns;
		TVertexInstanceAttributesRef<FVector4f> Colors;
		TVertexInstanceAttributesRef<FVector2f> UVs;

		// For read/write of Polygon attributes
		TPolygonAttributesRef<FName> PolygonObjectNames;  

		//GeometryParts
		FSkeletalMeshAttributes::FSourceGeometryPartNameRef GeometryPartNames;
		FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountRef GeometryPartVertexOffsetAndCounts;

		//For read only
		int32 NumUVChannels;
		int32 NumberOfMorphTargets;
		TArray<FName> MorphTargetNames; //Primarily important for the Source (or do we just move it outside of the helper?)
		TArray<TVertexAttributesRef<FVector3f>> MorphPositionDeltas; //Vertex, SourceMeshMorphPosDeltas[0] belongs to -> SourceMorphTargetNames[0] and so on..
		TArray<TVertexInstanceAttributesRef<FVector3f>> MorphNormalDeltas; //VertexInstance, SourceMeshMorphPosDeltas[0] belongs to -> SourceMorphTargetNames[0] and so on..

		FMeshAttributesHelper(FMeshDescription& SourceMeshDescription, const FMeshDescription* MeshDescriptionTemplate = nullptr)
			: MeshAttributes(SourceMeshDescription)
		{
			if (MeshDescriptionTemplate)
			{
				MeshAttributes.Register();

				FSkeletalMeshConstAttributes TemplateAttributes(*MeshDescriptionTemplate);
				if (TemplateAttributes.GetPolygonObjectNames().IsValid())
				{
					MeshAttributes.RegisterPolygonObjectNameAttribute();
				}
				if (TemplateAttributes.HasSourceGeometryParts())
				{
					MeshAttributes.RegisterSourceGeometryPartsAttributes();
				}

				for (const FName& MorphTargetName : TemplateAttributes.GetMorphTargetNames())
				{
					MeshAttributes.RegisterMorphTargetAttribute(MorphTargetName, TemplateAttributes.HasMorphTargetNormalsAttribute(MorphTargetName));
				}
			}

			PolygonGroupMaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();

			Positions = MeshAttributes.GetVertexPositions();
			SkinWeights = MeshAttributes.GetVertexSkinWeights();

			Normals = MeshAttributes.GetVertexInstanceNormals();
			Tangents = MeshAttributes.GetVertexInstanceTangents();
			BinormalSigns = MeshAttributes.GetVertexInstanceBinormalSigns();
			Colors = MeshAttributes.GetVertexInstanceColors();
			UVs = MeshAttributes.GetVertexInstanceUVs();

			PolygonObjectNames = MeshAttributes.GetPolygonObjectNames();

			GeometryPartNames = MeshAttributes.GetSourceGeometryPartNames();
			GeometryPartVertexOffsetAndCounts = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();

			NumUVChannels = UVs.GetNumChannels();

			MorphTargetNames = MeshAttributes.GetMorphTargetNames();

			NumberOfMorphTargets = MorphTargetNames.Num();
			MorphPositionDeltas.Reserve(NumberOfMorphTargets);
			MorphNormalDeltas.Reserve(NumberOfMorphTargets);
			for (const FName& MorphTargetName : MorphTargetNames)
			{
				MorphPositionDeltas.Add(MeshAttributes.GetVertexMorphPositionDelta(MorphTargetName));
				MorphNormalDeltas.Add(MeshAttributes.GetVertexInstanceMorphNormalDelta(MorphTargetName));
			}
		}
	};

	FMeshAttributesHelper Source(SourceMeshDescription);

	FMeshAttributesHelper Target(TargetMeshDescription, &SourceMeshDescription);

	const int32 TriangleCount = SourceMeshDescription.Triangles().Num();
	const int32 VertexInstanceCount = TriangleCount * 3;
	const int32 VertexCount = SourceMeshDescription.Vertices().Num();
	TargetMeshDescription.ReserveNewPolygonGroups(Source.PolygonGroupMaterialSlotNames.GetNumElements());
	TargetMeshDescription.ReserveNewPolygons(SourceMeshDescription.Polygons().Num());
	TargetMeshDescription.ReserveNewTriangles(TriangleCount);
	TargetMeshDescription.ReserveNewVertexInstances(VertexInstanceCount);
	TargetMeshDescription.ReserveNewVertices(VertexCount);

	Target.UVs.SetNumChannels(Source.NumUVChannels);

	TMap<FVertexID, FVertexID> SourceToTargetVertexIDMap;
	SourceToTargetVertexIDMap.Reserve(VertexCount);

	//Copy PolygonGroups and MaterialSlots:
	for (size_t PolygonGroupIndex = 0; PolygonGroupIndex < Source.PolygonGroupMaterialSlotNames.GetNumElements(); PolygonGroupIndex++)
	{
		FPolygonGroupID PolygonGroupID = TargetMeshDescription.CreatePolygonGroup();
		Target.PolygonGroupMaterialSlotNames[PolygonGroupID] = Source.PolygonGroupMaterialSlotNames[PolygonGroupIndex];
	}

	//Copy vertices (aka Position and SkinWeights)
	for (FVertexID SourceVertexID : SourceMeshDescription.Vertices().GetElementIDs())
	{
		FVertexID TargetVertexID = TargetMeshDescription.CreateVertex();

		//Position:
		Target.Positions[TargetVertexID] = Source.Positions[SourceVertexID];

		//SkinWeights:
		FVertexBoneWeights VertexBoneWeights = Source.SkinWeights.Get(SourceVertexID);
		TArray<UE::AnimationCore::FBoneWeight> BoneWeights;
		BoneWeights.Reserve(VertexBoneWeights.Num());
		for (const UE::AnimationCore::FBoneWeight BoneWeight : VertexBoneWeights)
		{
			BoneWeights.Add(BoneWeight);
		}
		Target.SkinWeights.Set(TargetVertexID, BoneWeights);

		SourceToTargetVertexIDMap.Add(SourceVertexID, TargetVertexID);
	}

	auto CopyVertexInstance = [&](FVertexInstanceID SourceVertexInstanceID, FVertexInstanceID TargetVertexInstanceID)
		{
			Target.Normals[TargetVertexInstanceID] = Source.Normals[SourceVertexInstanceID];
			Target.Tangents[TargetVertexInstanceID] = Source.Tangents[SourceVertexInstanceID];
			Target.BinormalSigns[TargetVertexInstanceID] = Source.BinormalSigns[SourceVertexInstanceID];
			Target.Colors[TargetVertexInstanceID] = Source.Colors[SourceVertexInstanceID];
			for (int32 UVIndex = 0; UVIndex < Source.NumUVChannels; ++UVIndex)
			{
				Target.UVs.Set(TargetVertexInstanceID, UVIndex, Source.UVs.Get(SourceVertexInstanceID, UVIndex));
			}
		};

	TargetFaceSmoothingMasks.SetNumZeroed(SourceMeshDescription.Triangles().Num());
	//Vertex Instances are created to be in complete order. so we don't copy over the existing VertexInstances prior to Triangle traverser.
	TArray<TPair<int32, int32>> SourceToTargetVertexInstanceIDs;
	SourceToTargetVertexInstanceIDs.Reserve(SourceMeshDescription.Triangles().Num() * 3);

	for (FPolygonID PolygonID : SourceMeshDescription.Polygons().GetElementIDs())
	{
		FPolygonGroupID PolygonGroupID = SourceMeshDescription.GetPolygonPolygonGroup(PolygonID);

		TArray<FVertexInstanceID> SourceVertexInstanceIDs = SourceMeshDescription.GetPolygonVertexInstances(PolygonID);
			
		TArray<FVertexInstanceID> TargetVertexInstanceIDs;
		TargetVertexInstanceIDs.SetNum(SourceVertexInstanceIDs.Num());

		for (int32 Corner = 0; Corner < SourceVertexInstanceIDs.Num(); ++Corner)
		{
			FVertexInstanceID SourceVertexInstanceID = SourceVertexInstanceIDs[Corner];

			FVertexID SourceParentVertexID = SourceMeshDescription.GetVertexInstanceVertex(SourceVertexInstanceID);
			FVertexID TargetParentVertexID = SourceToTargetVertexIDMap[SourceParentVertexID];

			FVertexInstanceID TargetVertexInstanceID = TargetMeshDescription.CreateVertexInstance(TargetParentVertexID);

			//Copy VertexInstance Values:
			CopyVertexInstance(SourceVertexInstanceID, TargetVertexInstanceID);

			TargetVertexInstanceIDs[Corner] = TargetVertexInstanceID;
			SourceToTargetVertexInstanceIDs.Add(TPair<int32, int32>(SourceVertexInstanceID, TargetVertexInstanceID)); //to be used for MorphTargetNormals copying
		}

		FPolygonID TargetPolygonId = TargetMeshDescription.CreatePolygon(PolygonGroupID, TargetVertexInstanceIDs);

		if (Target.PolygonObjectNames.IsValid())
		{
			Target.PolygonObjectNames.Set(TargetPolygonId, Source.PolygonObjectNames.Get(PolygonID));
		}

		//Smooth mask:
		TArrayView<const FTriangleID> SourceTriangleIDs = SourceMeshDescription.GetPolygonTriangles(PolygonID);
		TArrayView<const FTriangleID> TargetTriangleIDs = TargetMeshDescription.GetPolygonTriangles(TargetPolygonId);
		
		if (ensure(SourceTriangleIDs.Num() == TargetTriangleIDs.Num()))
		{
			for (int32 TriangleIndex = 0; TriangleIndex < SourceTriangleIDs.Num(); ++TriangleIndex)
			{
				if (SourceSmoothingMasks.IsValidIndex(SourceTriangleIDs[TriangleIndex].GetValue()))
				{
					if (ensure(TargetFaceSmoothingMasks.IsValidIndex(TargetTriangleIDs[TriangleIndex].GetValue())))
					{
						TargetFaceSmoothingMasks[TargetTriangleIDs[TriangleIndex].GetValue()] = SourceSmoothingMasks[SourceTriangleIDs[TriangleIndex].GetValue()];
					}
				}
			}
		}
	}

	//Morph Target Positions:
	for (size_t MorphTargetIndex = 0; MorphTargetIndex < Source.NumberOfMorphTargets; MorphTargetIndex++)
	{
		for (const TPair<FVertexID, FVertexID>& Entry : SourceToTargetVertexIDMap)
		{
			Target.MorphPositionDeltas[MorphTargetIndex][Entry.Value] = Source.MorphPositionDeltas[MorphTargetIndex][Entry.Key];
		}
	}

	//Copy the potential MorphTargetNormals:
	for (size_t MorphTargetIndex = 0; MorphTargetIndex < Source.NumberOfMorphTargets; MorphTargetIndex++)
	{
		if (Source.MorphNormalDeltas[MorphTargetIndex].IsValid())
		{
			for (const TPair<int32, int32>& Entry : SourceToTargetVertexInstanceIDs)
			{
				Target.MorphNormalDeltas[MorphTargetIndex][Entry.Value] = Source.MorphNormalDeltas[MorphTargetIndex][Entry.Key];
			}
		}
	}
	
	//Copy GeometryParts:
	for (const FSourceGeometryPartID GeometryPartID : Source.MeshAttributes.SourceGeometryParts().GetElementIDs())
	{
		FName Name = Source.GeometryPartNames.Get(GeometryPartID);
		TArrayView<const int32> OffsetAndCount = Source.GeometryPartVertexOffsetAndCounts.Get(GeometryPartID);

		FSourceGeometryPartID PartID = Target.MeshAttributes.CreateSourceGeometryPart();

		Target.GeometryPartNames.Set(PartID, Name);
		Target.GeometryPartVertexOffsetAndCounts.Set(PartID, { OffsetAndCount[0], OffsetAndCount[1]});
	}
};

void FSkeletalMeshOperations::ValidateFixComputeMeshDescriptionData(FMeshDescription& MeshDescription, const TArray<uint32>& FaceSmoothingMasks, int32 LODIndex, const bool bComputeWeightedNormals, const FString& SkeletalMeshPath)
{
	//As FSkeletalMeshImportData::GetMeshDescription(...) (which we replaced with direct FMeshDescription usage) is doing the following calls:
	// - ConvertSmoothGroupToHardEdges
	// - ValidateAndFixData
	// - HasInvalidVertexInstanceNormalsOrTangents -> ComputeTriangleTangentsAndNormals
	// - (Re)BuildIndexers
	//we are implementing the same mechanism here directly:
	{
		//TArray<uint32> FaceSmoothingMasks;
		//ConvertHardEdgesToSmoothMasks(MeshDescription, FaceSmoothingMasks);
		FSkeletalMeshOperations::ConvertSmoothGroupToHardEdges(FaceSmoothingMasks, MeshDescription);

		// Check if we have any broken data, including UVs and normals/tangents.
		FSkeletalMeshOperations::ValidateAndFixData(MeshDescription, SkeletalMeshPath);

		bool bHasInvalidNormals, bHasInvalidTangents;
		FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(MeshDescription, bHasInvalidNormals, bHasInvalidTangents);
		if (bHasInvalidNormals || bHasInvalidTangents)
		{
			// This is required by FSkeletalMeshOperations::ComputeTangentsAndNormals to function correctly.
			FSkeletalMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription, UE_SMALL_NUMBER, !SkeletalMeshPath.IsEmpty() ? *SkeletalMeshPath : nullptr);

			EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::None;
			if (bComputeWeightedNormals)
			{
				ComputeNTBsOptions |= EComputeNTBsFlags::WeightedNTBs;
			}

			// This only recomputes broken normals/tangents. The ValidateAndFixData function above will have turned all non-finite normals and tangents into
			// zero vectors.
			FSkeletalMeshOperations::ComputeTangentsAndNormals(MeshDescription, ComputeNTBsOptions);

			// We don't need the triangle tangents and normals anymore.
			MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Normal);
			MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Tangent);
			MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Binormal);
		}

		MeshDescription.RebuildIndexers();
	}
}

void FSkeletalMeshOperations::ValidateAndFixInfluences(FMeshDescription& MeshDescription, bool& bOutInfluenceCountLimitHit)
{
	//MeshDescription influences data validation and fixing:
	//													- Sort influences by weight and BoneIndex.
	//													- Normalize influence weights.
	//													- Warn about too many influences? (MAX_TOTAL_INFLUENCES)
	//													- Make sure all verts have influences set (if none exist bone 0 with weight 1)
	FSkeletalMeshAttributes MeshAttributes(MeshDescription);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttributes.GetVertexSkinWeights();

	bOutInfluenceCountLimitHit = false;

	if (!VertexSkinWeights.IsValid())
	{
		return;
	}

	for (FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
	{
		FVertexBoneWeights BoneWeights = VertexSkinWeights.Get(VertexID);
		const int32 InfluenceCount = BoneWeights.Num();

		if (InfluenceCount == 0)
		{
			VertexSkinWeights.Set(VertexID, { UE::AnimationCore::FBoneWeight(0, 1.f) });
			continue;
		}

		TArray<UE::AnimationCore::FBoneWeight> BoneWeightArray;
		BoneWeightArray.Reserve(InfluenceCount);

		float TotalWeight = 0.0f;
		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			BoneWeightArray.Add(BoneWeights[InfluenceIndex]);
			TotalWeight += BoneWeights[InfluenceIndex].GetWeight();
		}

		if (InfluenceCount && (TotalWeight != 1.0f))
		{
			float OneOverTotalWeight = 1.f / TotalWeight;

			for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
			{
				BoneWeightArray[InfluenceIndex].SetWeight(BoneWeights[InfluenceIndex].GetWeight() * OneOverTotalWeight);
			}
		}

		if (InfluenceCount > MAX_TOTAL_INFLUENCES)
		{
			bOutInfluenceCountLimitHit = true;
		}

		BoneWeightArray.Sort([](const UE::AnimationCore::FBoneWeight& A, const UE::AnimationCore::FBoneWeight& B)
			{
				if (A.GetWeight() < B.GetWeight()) return false;
				else if (A.GetWeight() > B.GetWeight()) return true;
				else if (A.GetBoneIndex() > B.GetBoneIndex()) return false;
				else if (A.GetBoneIndex() < B.GetBoneIndex()) return true;
				else return false;
			});


		for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
		{
			BoneWeights[InfluenceIndex].SetBoneIndex(BoneWeightArray[InfluenceIndex].GetBoneIndex());
			BoneWeights[InfluenceIndex].SetRawWeight(BoneWeightArray[InfluenceIndex].GetRawWeight());
		}
	}
}

namespace RigApplicationHelpers
{
	struct FVertexInfo
	{
		FVector3f Position;
		FVertexID VertexID;

		FVertexInfo(const FVector3f& InPosition, const FVertexID InVertexID)
			: Position(InPosition)
			, VertexID(InVertexID)
		{
		}
	};

	/** Helper struct for the mesh component vert position octree */
	struct FVertexInfoOctreeSemantics
	{
		enum { MaxElementsPerLeaf = 16 };
		enum { MinInclusiveElementsPerNode = 7 };
		enum { MaxNodeDepth = 12 };

		typedef TInlineAllocator<MaxElementsPerLeaf> ElementAllocator;

		/**
		* Get the bounding box of the provided octree element. In this case, the box
		* is merely the point specified by the element.
		*
		* @param	Element	Octree element to get the bounding box for
		*
		* @return	Bounding box of the provided octree element
		*/
		FORCEINLINE static FBoxCenterAndExtent GetBoundingBox(const FVertexInfo& Element)
		{
			return FBoxCenterAndExtent(FVector(Element.Position), FVector::ZeroVector);
		}

		/**
		* Determine if two octree elements are equal
		*
		* @param	A	First octree element to check
		* @param	B	Second octree element to check
		*
		* @return	true if both octree elements are equal, false if they are not
		*/
		FORCEINLINE static bool AreElementsEqual(const FVertexInfo& A, const FVertexInfo& B)
		{
			return (A.Position == B.Position && A.VertexID == B.VertexID);
		}

		/** Ignored for this implementation */
		FORCEINLINE static void SetElementId(const FVertexInfo& Element, FOctreeElementId2 Id)
		{
		}
	};
	typedef TOctree2<FVertexInfo, FVertexInfoOctreeSemantics> TVertexInfoPosOctree;

	/** Helper struct for building acceleration structures. */
	struct FIndexAndZ
	{
		float Z;
		int32 Index;

		/** Initialization constructor. */
		FIndexAndZ(int32 InIndex, const FVector3f& V)
		{
			Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;
			Index = InIndex;
		}
	};

	/** Sorting function for vertex Z/index pairs. */
	struct FCompareIndexAndZ
	{
		FORCEINLINE bool operator()(FIndexAndZ const& A, FIndexAndZ const& B) const { return A.Z < B.Z; }
	};

	inline bool PointsEqual(const FVector3f& V1, const FVector3f& V2, float ComparisonThreshold)
	{
		if (FMath::Abs(V1.X - V2.X) > ComparisonThreshold
			|| FMath::Abs(V1.Y - V2.Y) > ComparisonThreshold
			|| FMath::Abs(V1.Z - V2.Z) > ComparisonThreshold)
		{
			return false;
		}
		return true;
	}

	void FindMatchingPositionVertexIndexes(const FVector3f& Position,
		const TArray<FIndexAndZ>& SortedPositions, TVertexAttributesRef<const FVector3f> RigVertexPositions,
		float ComparisonThreshold,
		TArray<int32>& OutResults)
	{
		int32 SortedPositionNumber = SortedPositions.Num();
		OutResults.Empty();
		if (SortedPositionNumber == 0)
		{
			//No possible match
			return;
		}
		FIndexAndZ PositionIndexAndZ(INDEX_NONE, Position);
		int32 SortedIndex;
		int32 LastTopIndex = SortedPositions.Num();
		int32 LastBottomIndex = 0;

		{
			double Increments = ((double)SortedPositions[SortedPositionNumber - 1].Z - (double)SortedPositions[0].Z) / (double)SortedPositionNumber;

			//Optimize the iteration count when a value is not in the middle
			if (!FMath::IsNearlyZero(Increments))
			{
				SortedIndex = FMath::RoundToInt(((double)PositionIndexAndZ.Z - (double)SortedPositions[0].Z) / Increments);
			}
			else
			{
				SortedIndex = LastTopIndex;
			}
		}

		for (int32 SearchIterationCount = 0; SortedPositions.IsValidIndex(SortedIndex); ++SearchIterationCount)
		{
			if (LastTopIndex - LastBottomIndex < 5)
			{
				break;
			}
			if (FMath::Abs(PositionIndexAndZ.Z - SortedPositions[SortedIndex].Z) < ComparisonThreshold)
			{
				//Continue since we want the lowest start
				LastTopIndex = SortedIndex;
				SortedIndex = LastBottomIndex + ((LastTopIndex - LastBottomIndex) / 2);
				if (SortedIndex <= LastBottomIndex)
				{
					break;
				}
			}
			else if (PositionIndexAndZ.Z > SortedPositions[SortedIndex].Z + ComparisonThreshold)
			{
				LastBottomIndex = SortedIndex;
				SortedIndex = SortedIndex + FMath::Max(((LastTopIndex - SortedIndex) / 2), 1);
			}
			else
			{
				LastTopIndex = SortedIndex;
				SortedIndex = SortedIndex - FMath::Max(((SortedIndex - LastBottomIndex) / 2), 1);
			}
		}

		//////////////////////////////////////////////////////////////////////////
		//Closest point data (!bExactMatch)
		for (int32 i = LastBottomIndex; i < SortedPositionNumber; i++)
		{
			//Get fast to the close position
			if (PositionIndexAndZ.Z > SortedPositions[i].Z + ComparisonThreshold)
			{
				continue;
			}
			//break when we pass point close to the position
			if (SortedPositions[i].Z > PositionIndexAndZ.Z + ComparisonThreshold)
				break; // can't be any more dups

			//Point is close to the position, verify it
			const FVector3f& PositionA = RigVertexPositions[SortedPositions[i].Index];
			if (PointsEqual(PositionA, Position, ComparisonThreshold))
			{
				OutResults.Add(SortedPositions[i].Index);
			}
		}
	}

	float GetSmallestDeltaBetweenTriangleLists(TArray<const FTriangleID>* RigTriangles, TArray<const FTriangleID>* GeoTriangles,
		TTriangleAttributesRef<TArrayView<const FVertexID>> RigTriangleVertices, TTriangleAttributesRef<TArrayView<const FVertexID>> GeoTriangleVertices,
		TVertexAttributesRef<const FVector3f> RigVertexPositions, TVertexAttributesRef<const FVector3f> GeoVertexPositions)
	{
		float SmallestTriangleDeltaSum = FLT_MAX;

		for (const FTriangleID& RigTriangle : *RigTriangles)
		{
			const FVector3f RigPointA = RigVertexPositions[RigTriangleVertices[RigTriangle][0]];
			const FVector3f RigPointB = RigVertexPositions[RigTriangleVertices[RigTriangle][1]];
			const FVector3f RigPointC = RigVertexPositions[RigTriangleVertices[RigTriangle][2]];

			for (const FTriangleID& GeoTriangle : *GeoTriangles)
			{
				const FVector3f GeoPointA = GeoVertexPositions[GeoTriangleVertices[GeoTriangle][0]];
				const FVector3f GeoPointB = GeoVertexPositions[GeoTriangleVertices[GeoTriangle][1]];
				const FVector3f GeoPointC = GeoVertexPositions[GeoTriangleVertices[GeoTriangle][2]];

				float TriangleDeltaSum =
					(GeoPointA - RigPointA).Size() +
					(GeoPointB - RigPointB).Size() +
					(GeoPointC - RigPointC).Size();

				if (SmallestTriangleDeltaSum > TriangleDeltaSum)
				{
					SmallestTriangleDeltaSum = TriangleDeltaSum;
				}
			}
		}

		return SmallestTriangleDeltaSum;
	}

	void FindNearestVertexIndices(TVertexInfoPosOctree& VertexInfoPosOctree, const FVector3f& SearchPosition, TArray<FVertexInfo>& OutNearestVertices)
	{
		OutNearestVertices.Empty();
		const float OctreeExtent = VertexInfoPosOctree.GetRootBounds().Extent.Size3();
		//Use the max between 1e-4 cm and 1% of the bounding box extend
		FVector Extend(FMath::Max(UE_KINDA_SMALL_NUMBER, OctreeExtent * 0.005f));

		//Pass Extent size % of the Octree bounding box extent
		//PassIndex 0 -> 0.5%
		//PassIndex n -> 0.05*n
		//PassIndex 1 -> 5%
		//PassIndex 2 -> 10%
		//...
		for (int32 PassIndex = 0; PassIndex < 5; ++PassIndex)
		{
			// Query the octree to find the vertices close(inside the extend) to the SearchPosition
			VertexInfoPosOctree.FindElementsWithBoundsTest(FBoxCenterAndExtent((FVector)SearchPosition, Extend), [&OutNearestVertices](const FVertexInfo& VertexInfo)
				{
					// Add all of the elements in the current node to the list of points to consider for closest point calculations
					OutNearestVertices.Add(VertexInfo);
				});
			if (OutNearestVertices.Num() == 0)
			{
				float ExtentPercent = 0.05f * ((float)PassIndex + 1.0f);
				Extend = FVector(FMath::Max(UE_KINDA_SMALL_NUMBER, OctreeExtent * ExtentPercent));
			}
			else
			{
				break;
			}
		}
	}
}// namespace RigApplicationHelpers

void FSkeletalMeshOperations::ApplyRigToGeo(FMeshDescription& RigMeshDescription /*Base/From/'Other'*/, FMeshDescription& GeoMeshDescription /*Target/To*/)
{
	//Primarily used for "bApplyGeometryOnly" and "bApplySkinningOnly"
	//ExistingMeshDescription vs NewMeshDescription
	//both consists of "Geometry" and "Skinning"
	//IF bApplyGeometryOnly is active:
	//		then we want to apply the Rig (aka skinning) from the ExistingMeshDescription to the NewMeshDescription and hence the created 'new' NewMeshDescription is going be the result.
	//IF bApplySkinningOnly is active:
	//		then we want to apply the Rig (aka skinning) from the NewMeshDescription to the ExistingMeshDescription and hence the created 'new' ExistingMeshDescription is going to be the result.

	using namespace RigApplicationHelpers;

	FSkeletalMeshAttributes RigAttributes(RigMeshDescription);
	FSkeletalMeshAttributes GeoAttributes(GeoMeshDescription);

	TMap<const FVertexID, TArray<const FTriangleID>> RigVertexToTriangleIDs;
	TMap<const FVertexID, TArray<const FTriangleID>> GeoVertexToTriangleIDs;

	TTriangleAttributesRef<TArrayView<const FVertexID>> RigTriangleVertices = RigAttributes.GetTriangleVertexIndices();
	TTriangleAttributesRef<TArrayView<const FVertexID>> GeoTriangleVertices = GeoAttributes.GetTriangleVertexIndices();

	TVertexAttributesRef<const FVector3f> RigVertexPositions = RigAttributes.GetVertexPositions();
	TVertexAttributesRef<const FVector3f> GeoVertexPositions = GeoAttributes.GetVertexPositions();

	//Build look up table from vertexid to triangles:
	for (FTriangleID TriangleID : RigMeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> VertexIndices = RigTriangleVertices[TriangleID];

		for (const FVertexID& VertexID : VertexIndices)
		{
			TArray<const FTriangleID>& Triangles = RigVertexToTriangleIDs.FindOrAdd(VertexID);
			Triangles.Add(TriangleID);
		}
	}
	for (FTriangleID TriangleID : GeoMeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> VertexIndices = GeoTriangleVertices[TriangleID];

		for (const FVertexID& VertexID : VertexIndices)
		{
			TArray<const FTriangleID>& Triangles = GeoVertexToTriangleIDs.FindOrAdd(VertexID);
			Triangles.Add(TriangleID);
		}
	}

	// Find the extents formed by the cached vertex positions in order to optimize the octree used later
	FBox3f Bounds(ForceInitToZero);

	for (const FVertexID VertexID : RigMeshDescription.Vertices().GetElementIDs())
	{
		const FVector3f& Position = RigVertexPositions[VertexID];
		Bounds += Position;
	}
	for (const FVertexID VertexID : GeoMeshDescription.Vertices().GetElementIDs())
	{
		const FVector3f& Position = GeoVertexPositions[VertexID];
		Bounds += Position;
	}

	//Init Octree and SortedPositions.
	TVertexInfoPosOctree RigVertPosOctree(FVector(Bounds.GetCenter()), Bounds.GetExtent().GetMax());
	TArray<FIndexAndZ> SortedPositions;

	//Adding the rig's geometry to the oct tree.
	//As we are going to look for Geometry's positions in the Rig's Geometry in order to find the closes match for the influence to be applied:
	for (const FVertexID VertexID : RigMeshDescription.Vertices().GetElementIDs())
	{
		const FVector3f& Position = RigVertexPositions[VertexID];
		RigVertPosOctree.AddElement(FVertexInfo(Position, VertexID));
		SortedPositions.Add(FIndexAndZ(VertexID.GetValue(), Position));
	}

	SortedPositions.Sort(FCompareIndexAndZ());

	//Start finding influences from Rig for Geo:
	TMap<int32, int32> GeoToRigMap;
	GeoToRigMap.Reserve(RigMeshDescription.Vertices().Num());

	for (const FVertexID VertexID : GeoMeshDescription.Vertices().GetElementIDs())
	{
		TArray<const FTriangleID>* GeoTriangles = GeoVertexToTriangleIDs.Find(VertexID);

		if (!GeoTriangles)
		{
			continue;
		}

		const FVector3f& SearchPosition = GeoVertexPositions[VertexID]; //Position to match.

		//Important Note: FSkeletalMeshImportData::ApplyRigToGeo seem to work based on VertexInstances, it also checks the VertexCandidate Normal and UVs and only finds the candidate legitimate if they match between Rig and Geo)
		//		As Influences (BoneIndex and BoneWeights) are Vertex (NOT VertexInstance) dependent.
		//		Whilst original implementation in FSkeletalMeshImportData::ApplyRigToGeo was checking and validating against normals and UVs for NearestWedges,
		//		with current implementation we try the NearestVertices with the same principle as the FindMatchingPositionVertexIndexes. (aka based on GetSmallestDeltaBetweenTriangleLists)

		//First we look for identical matches: (aka FWedgePosition::FindMatchingPositionWegdeIndexes)
		TArray<int32> VertexCandidates; //Vertex IDs from Rig.
		FindMatchingPositionVertexIndexes(SearchPosition,
			SortedPositions, RigVertexPositions,
			UE_THRESH_POINTS_ARE_SAME,
			VertexCandidates);

		bool bFoundMatch = false;

		if (VertexCandidates.Num() > 0)
		{
			int32 BestVertexIndexCandidate = INDEX_NONE; //From Rig
			float LowestTriangleDeltaSum = 0;

			for (int32 VertexCandidate : VertexCandidates)
			{
				TArray<const FTriangleID>* RigTriangles = RigVertexToTriangleIDs.Find(VertexCandidate);

				if (!RigTriangles)
				{
					continue;
				}

				float CandidateSmallestTriangleDelta = GetSmallestDeltaBetweenTriangleLists(RigTriangles, GeoTriangles, RigTriangleVertices, GeoTriangleVertices, RigVertexPositions, GeoVertexPositions);

				if (BestVertexIndexCandidate == INDEX_NONE || LowestTriangleDeltaSum > CandidateSmallestTriangleDelta)
				{
					BestVertexIndexCandidate = VertexCandidate;
					LowestTriangleDeltaSum = CandidateSmallestTriangleDelta;
				}
			}

			if (BestVertexIndexCandidate != INDEX_NONE)
			{
				GeoToRigMap.Add(VertexID, BestVertexIndexCandidate); //Aka mapping the geometry vertex id to the rig vertex id, as to which righ vertex id is the best one to use for influence.
				bFoundMatch = true;
			}
		}

		if (!bFoundMatch)
		{
			//In case exact matching didn't produce a result,
			//then we do FindNearestWedgeIndexes.

			int32 BestVertexIndexCandidate = INDEX_NONE; //From Rig
			float LowestTriangleDeltaSum = 0;

			TArray<FVertexInfo> NearestVertices;
			FindNearestVertexIndices(RigVertPosOctree, SearchPosition, NearestVertices);

			for (const FVertexInfo& VertexInfoCandidate : NearestVertices)
			{
				int32 VertexCandidate = VertexInfoCandidate.VertexID;
				TArray<const FTriangleID>* RigTriangles = RigVertexToTriangleIDs.Find(VertexCandidate);

				if (!RigTriangles)
				{
					continue;
				}

				float CandidateSmallestTriangleDelta = GetSmallestDeltaBetweenTriangleLists(RigTriangles, GeoTriangles, RigTriangleVertices, GeoTriangleVertices, RigVertexPositions, GeoVertexPositions);

				if (BestVertexIndexCandidate == INDEX_NONE || LowestTriangleDeltaSum > CandidateSmallestTriangleDelta)
				{
					BestVertexIndexCandidate = VertexCandidate;
					LowestTriangleDeltaSum = CandidateSmallestTriangleDelta;
				}
			}

			if (BestVertexIndexCandidate != INDEX_NONE)
			{
				GeoToRigMap.Add(VertexID, BestVertexIndexCandidate); //Aka mapping the geometry vertex id to the rig vertex id, as to which righ vertex id is the best one to use for influence.
			}
		}
	}

	FSkinWeightsVertexAttributesRef RigVertexSkinWeights = RigAttributes.GetVertexSkinWeights();
	FSkinWeightsVertexAttributesRef GeoVertexSkinWeights = GeoAttributes.GetVertexSkinWeights();
	for (const FVertexID VertexID : GeoMeshDescription.Vertices().GetElementIDs())
	{
		const int32* RigVertexID = GeoToRigMap.Find(VertexID);

		if (RigVertexID)
		{
			FVertexBoneWeights VertexBoneWeights = RigVertexSkinWeights.Get(*RigVertexID);

			TArray<UE::AnimationCore::FBoneWeight> BoneWeights;
			for (const UE::AnimationCore::FBoneWeight BoneWeight : VertexBoneWeights)
			{
				BoneWeights.Add(BoneWeight);
			}
			GeoVertexSkinWeights.Set(VertexID, BoneWeights);
		}
		else
		{
			//if the VertexID does not have a mapping, then set boneIndex 0 with weight 1.
			GeoVertexSkinWeights.Set(VertexID, { UE::AnimationCore::FBoneWeight(0, 1.f) });
		}
	}
}

void FSkeletalMeshOperations::PruneBoneWeights(
	FMeshDescription& InOutMeshDescription,
	const UE::AnimationCore::FBoneWeightsSettings& InBoneWeightSettings,
	TConstArrayView<FName> InProfileNames
	)
{
	FSkeletalMeshAttributes Attributes(InOutMeshDescription);
	TArray<FName> ProfileNames = Attributes.GetSkinWeightProfileNames();

	if (!InProfileNames.IsEmpty())
	{
		ProfileNames = ProfileNames.FilterByPredicate([&InProfileNames](const FName ProfileName) -> bool
			{
				return InProfileNames.Contains(ProfileName);
			});
	}
	
	ParallelFor(ProfileNames.Num(), [&InOutMeshDescription, &InBoneWeightSettings, &ProfileNames](int32 ProfileIndex)
		{
			using namespace UE::AnimationCore;
			
			const FName ProfileName = ProfileNames[ProfileIndex];
			FSkeletalMeshAttributes Attributes(InOutMeshDescription);
			FSkinWeightsVertexAttributesRef SkinWeights = Attributes.GetVertexSkinWeights(ProfileName);

			for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
			{
				FVertexBoneWeights VertexBoneWeights = SkinWeights.Get(VertexID);

				VertexBoneWeights.Renormalize(InBoneWeightSettings);
			}
		});
}


void FSkeletalMeshOperations::PartitionByMaximumUniqueBoneIndexCount(
	FMeshDescription& InOutMeshDescription,
	const int32 InMaxBoneCount,
	TConstArrayView<FName> InProfileNames
	)
{
	if (InOutMeshDescription.Triangles().Num() == 0)
	{
		return;
	}

	// If the number of unique bones can never exceed the given limit, bail out already.
	FSkeletalMeshAttributes Attributes(InOutMeshDescription);
	if (!Attributes.HasBones() || Attributes.GetNumBones() <= InMaxBoneCount)
	{
		return;
	}

	TArray<FName> ProfileNames = Attributes.GetSkinWeightProfileNames();

	if (!InProfileNames.IsEmpty())
	{
		ProfileNames = ProfileNames.FilterByPredicate([&InProfileNames](const FName ProfileName) -> bool
			{
				return InProfileNames.Contains(ProfileName);
			});
	}
	
	TArray<FSkinWeightsVertexAttributesRef> SkinWeightAttributes;
	for (FName ProfileName : ProfileNames)
	{
		SkinWeightAttributes.Add(Attributes.GetVertexSkinWeights(ProfileName));
	}
	
	struct FTrianglePatch
	{
		TArray<FTriangleID> Triangles;
		TSet<FBoneIndexType> UniqueBoneSet;
	};

	TArray<TArray<FTrianglePatch>> TrianglePatchGroups;
	TArray<TArrayView<const FTriangleID>> GroupedTriangles;

	TrianglePatchGroups.SetNum(InOutMeshDescription.PolygonGroups().GetArraySize());
	GroupedTriangles.SetNum(InOutMeshDescription.PolygonGroups().GetArraySize());

	for (FPolygonGroupID PolygonGroupID: InOutMeshDescription.PolygonGroups().GetElementIDs())
	{
		GroupedTriangles[PolygonGroupID] = InOutMeshDescription.GetPolygonGroupTriangles(PolygonGroupID);
	}
	
	// Traverse neighboring triangles, of all the same polygon group, and add them to the same triangle patch
	// until we exceed the list of unique bone indices for that patch, then start a new patch for the same polygon group.
	ParallelFor(InOutMeshDescription.PolygonGroups().GetArraySize(), [&TrianglePatchGroups, &GroupedTriangles, &Mesh=InOutMeshDescription, &SkinWeightAttributes, InMaxBoneCount](int32 InGroupIndex) -> void
		{
			FPolygonGroupID PolygonGroupID(InGroupIndex);
			if (!Mesh.PolygonGroups().IsValid(PolygonGroupID))
			{
				return;
			}

			TArrayView<const FTriangleID> GroupTriangles = GroupedTriangles[InGroupIndex];
			if (GroupTriangles.IsEmpty())
			{
				return;
			}
			
			TBitArray<> VisitedTriangles;
			TDeque<FTriangleID> TriangleQueue;
			VisitedTriangles.SetNum(Mesh.Triangles().GetArraySize(), false);

			int32 CurrentTriangleIndex = 0;
			TriangleQueue.PushLast(GroupTriangles[CurrentTriangleIndex]);
			VisitedTriangles[GroupTriangles[CurrentTriangleIndex]] = true;
			CurrentTriangleIndex++;

			TSet<FBoneIndexType> TriangleBoneSet;
			
			auto PopulateBoneSetForTriangle = [&SkinWeightAttributes, &Mesh, &TriangleBoneSet](FTriangleID InTriangleID) -> void
				{
					TriangleBoneSet.Empty();
					for (const FVertexID VertexID: Mesh.GetTriangleVertices(InTriangleID))
					{
						for (FSkinWeightsVertexAttributesRef SkinWeightAttribute: SkinWeightAttributes)
						{
							for (UE::AnimationCore::FBoneWeight BoneWeight: SkinWeightAttribute.Get(VertexID))
							{
								TriangleBoneSet.Add(BoneWeight.GetBoneIndex());
							}
						}
					}
				};

			while (!TriangleQueue.IsEmpty())
			{
				FTriangleID TriangleID = TriangleQueue.First();
				TriangleQueue.PopFirst();
				FPolygonGroupID GroupID = Mesh.GetTrianglePolygonGroup(TriangleID);

				TArray<FTrianglePatch>& PatchListForGroup = TrianglePatchGroups[GroupID];
				if (PatchListForGroup.IsEmpty())
				{
					PatchListForGroup.AddDefaulted();
				}
				FTrianglePatch& CurrentPatch = PatchListForGroup.Last();
				
				PopulateBoneSetForTriangle(TriangleID);
				if (CurrentPatch.UniqueBoneSet.Includes(TriangleBoneSet))
				{
					CurrentPatch.Triangles.Add(TriangleID);
				}
				else
				{
					// If the union of the patch's bone set and the triangle's bone set exceeds the maximum, start a new patch
					// and reset the triangle queue. That way we can start a new seed patch that grows radially from the seed 
					// triangle, rather than just cleaning up whatever disjointed triangles might remain in the queue.
					TSet<FBoneIndexType> CombinedBoneSet = CurrentPatch.UniqueBoneSet;
					CombinedBoneSet.Append(TriangleBoneSet);
					if (CombinedBoneSet.Num() > InMaxBoneCount)
					{
						FTrianglePatch& NewPatch = PatchListForGroup.AddDefaulted_GetRef();
						NewPatch.UniqueBoneSet = TriangleBoneSet;
						NewPatch.Triangles.Add(TriangleID);

						// Mark all triangles in the queue as unvisited before we clear it.
						for (const FTriangleID QueuedTriangleID: TriangleQueue)
						{
							VisitedTriangles[QueuedTriangleID] = false;
						}
						TriangleQueue.Empty();
						
					}
					else
					{
						CurrentPatch.UniqueBoneSet = MoveTemp(CombinedBoneSet);
						CurrentPatch.Triangles.Add(TriangleID);
					}
				}

				// First try to find neighbors of the same polygon group ID and add them to the queue.
				// The reason we stick with same polygon group ID is so that we can try to maintain as
				// large of a contiguous patch as possible. If we just spread over all possible polygon IDs
				// the traversal would end up being more disjointed.
				for (FEdgeID EdgeID : Mesh.GetTriangleEdges(TriangleID))
				{
					for (FTriangleID ConnectedTriangleID : Mesh.GetEdgeConnectedTriangleIDs(EdgeID))
					{
						if (ConnectedTriangleID != TriangleID && !VisitedTriangles[ConnectedTriangleID])
						{
							FPolygonGroupID ConnectedGroupID = Mesh.GetTrianglePolygonGroup(ConnectedTriangleID);
							if (ConnectedGroupID == GroupID)
							{
								TriangleQueue.PushLast(ConnectedTriangleID);
								VisitedTriangles[ConnectedTriangleID] = true;
							}
						}
					}
				}

				// We didn't find a neighboring triangle to add. Go down the list of triangles until we find one we haven't
				// visited yet.
				if (TriangleQueue.IsEmpty())
				{
					while (CurrentTriangleIndex < GroupTriangles.Num() && VisitedTriangles[GroupTriangles[CurrentTriangleIndex]])
					{
						++CurrentTriangleIndex;
					}

					if (CurrentTriangleIndex != GroupTriangles.Num())
					{
						// Create a new patch group for this new set of triangles. This helps with merging later.
						PatchListForGroup.AddDefaulted();
						
						TriangleQueue.PushLast(GroupTriangles[CurrentTriangleIndex]);
						VisitedTriangles[GroupTriangles[CurrentTriangleIndex]] = true;
						++CurrentTriangleIndex;
					}
				}
			}
		});

	// We now have a bunch of patches. Let's see if we can combine some of them, to minimize their number.
	ParallelFor(TrianglePatchGroups.Num(), [&TrianglePatchGroups, InMaxBoneCount](int32 InGroupIndex)
		{
			TArray<FTrianglePatch>& PatchListForGroup = TrianglePatchGroups[InGroupIndex];
			if (PatchListForGroup.Num() <= 1)
			{
				return;
			}

			// Iteratively go through each patch pair, calculate a distance score between them and merge patches with 
			// the lowest score until we run out of patches to merge.
			TArray<TPair<int32 /* Patch Index */, int32 /* Distance Score */>> CandidatePatchDistances;
			CandidatePatchDistances.Reserve(PatchListForGroup.Num());
			
			for (int32 PatchIndexA = 0; PatchIndexA < (PatchListForGroup.Num() - 1); ++PatchIndexA)
			{
				FTrianglePatch& TargetPatch = PatchListForGroup[PatchIndexA];
				if (TargetPatch.Triangles.IsEmpty())
				{
					continue;
				}

				CandidatePatchDistances.Empty();
				
				for (int32 PatchIndexB = PatchIndexA + 1; PatchIndexB < PatchListForGroup.Num(); ++PatchIndexB)
				{
					FTrianglePatch& MergePatch = PatchListForGroup[PatchIndexB];
					if (MergePatch.Triangles.IsEmpty())
					{
						continue;
					}

					CandidatePatchDistances.Emplace(PatchIndexB, 0);
					for (FBoneIndexType BoneIndex : MergePatch.UniqueBoneSet)
					{
						if (!TargetPatch.UniqueBoneSet.Contains(BoneIndex))
						{
							CandidatePatchDistances.Last().Value++;
						}
					}
				}

				// Sort by score and merge until we exceed the unique bone count
				CandidatePatchDistances.StableSort([](const TPair<int32, int32>& A, const TPair<int32, int32>& B)
					{
						return A.Value < B.Value;
					});

				for (const TPair<int32, int32>& PatchDistance: CandidatePatchDistances)
				{
					FTrianglePatch& MergePatch = PatchListForGroup[PatchDistance.Key];
					TSet<FBoneIndexType> CandidateBoneSet = TargetPatch.UniqueBoneSet.Union(MergePatch.UniqueBoneSet);
					if (CandidateBoneSet.Num() <= InMaxBoneCount)
					{
						TargetPatch.Triangles.Append(MergePatch.Triangles);
						TargetPatch.UniqueBoneSet = MoveTemp(CandidateBoneSet);
						MergePatch.Triangles.Empty();
					}
				}
			}
		});

	// The number of non-empty patches, less the first one in each group, will give us the number of
	// extra polygon groups we will need.
	int32 RequiredNewGroupCount = 0;
	for (const TArray<FTrianglePatch>& PatchListForGroup : TrianglePatchGroups)
	{
		for (int32 PatchIndex = 1; PatchIndex < PatchListForGroup.Num(); ++PatchIndex)
		{
			const FTrianglePatch& TrianglePatch = PatchListForGroup[PatchIndex];
			if (!TrianglePatch.Triangles.IsEmpty())
			{
				RequiredNewGroupCount++;
			}
		}
	}

	// If there's no change to the group count, then we're good.
	if (RequiredNewGroupCount == 0)
	{
		return;
	}
	
	// If we have more than one patch per polygon group, re-assign all patches, save for the first one in a group, to
	// a new polygon group but indicate where it got spawned off from.
	Attributes.RegisterPolygonGroupParentAttribute();

	InOutMeshDescription.ReserveNewPolygonGroups(RequiredNewGroupCount);

	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TPolygonGroupAttributesRef<int32> PolygonGroupParents = Attributes.GetPolygonGroupParents();
	
	InOutMeshDescription.SuspendPolygonGroupIndexing();

	// PatchGroupIndex is the actual polygon group on the original mesh.
	for (int32 PatchGroupIndex = 0; PatchGroupIndex < TrianglePatchGroups.Num(); ++PatchGroupIndex)
	{
		const TArray<FTrianglePatch>& PatchListForGroup = TrianglePatchGroups[PatchGroupIndex];
		if (PatchListForGroup.IsEmpty())
		{
			continue;
		}

		FPolygonGroupID PolygonGroupID(PatchGroupIndex);
		FName MaterialName = PolygonGroupImportedMaterialSlotNames[PolygonGroupID];
		
		for (int32 PatchIndex = 1; PatchIndex < PatchListForGroup.Num(); ++PatchIndex)
		{
			const FTrianglePatch& TrianglePatch = PatchListForGroup[PatchIndex];
			if (TrianglePatch.Triangles.IsEmpty())
			{
				continue;
			}
			
			const FPolygonGroupID PatchPolygonGroupID = InOutMeshDescription.CreatePolygonGroup();
			
			PolygonGroupParents[PatchPolygonGroupID] = PolygonGroupID;
			PolygonGroupImportedMaterialSlotNames[PatchPolygonGroupID] = MaterialName;

			for (const FTriangleID TriangleID: TrianglePatch.Triangles)
			{
				const FPolygonID PolygonID = InOutMeshDescription.GetTrianglePolygon(TriangleID);
				InOutMeshDescription.SetPolygonPolygonGroup(PolygonID, PatchPolygonGroupID);
			}
		}
	}

	InOutMeshDescription.ResumePolygonGroupIndexing();
	InOutMeshDescription.BuildPolygonGroupIndexers();
}


void FSkeletalMeshOperations::RemoveBones(
	FMeshDescription& InOutMeshDescription, 
	TConstArrayView<FName> InBoneNames,
	const UE::AnimationCore::FBoneWeightsSettings& InBoneWeightSettings,
	TConstArrayView<FName> InProfileNames
	)
{
	FSkeletalMeshAttributes Attributes(InOutMeshDescription);
	
	// No bone names to match against -- can't do any further processing.
	if (InBoneNames.IsEmpty() || !Attributes.HasBones() || !Attributes.HasBoneNameAttribute() || !Attributes.HasBoneParentIndexAttribute())
	{
		return;
	}
	
	// Create a map of the bone hierarchy, keyed on bone name. Each value contains the parent bone name,
	// the original index and whether the bone is currently in use.
	// The bone attributes are assumed to be consistent and sane.
	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNamesAttrib = Attributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentsAttrib = Attributes.GetBoneParentIndices();
	
	struct FBoneInfo
	{
		FName ParentBoneName;
		bool bToRemove = false;
		FBoneID OriginalID;
		TArray<FName> Children;
	};
	
	TMap<FName, FBoneInfo> BoneInfoMap;
	FName RootBoneName;
	
	for (FBoneID BoneID: Attributes.Bones().GetElementIDs())
	{
		const FName BoneName = BoneNamesAttrib.Get(BoneID);
		const int32 ParentBoneIndex = BoneParentsAttrib.Get(BoneID);
		
		FBoneInfo BoneInfo;
		BoneInfo.ParentBoneName = Attributes.Bones().IsValid(ParentBoneIndex) ? BoneNamesAttrib.Get(ParentBoneIndex) : NAME_None;
		BoneInfo.OriginalID = BoneID.GetValue();
		BoneInfoMap.Emplace(BoneName, BoneInfo);
		
		if (BoneInfo.ParentBoneName == NAME_None)
		{
			if (RootBoneName != NAME_None)
			{
				UE_LOGF(LogSkeletalMeshOperations, Error, "Multiple root bones found: %ls and %ls", *RootBoneName.ToString(), *BoneName.ToString());
				return;
			}
			RootBoneName = BoneName;
		}
	}
	
	// Fill in the list of children for each joint.
	for (const TPair<FName, FBoneInfo>& Item: BoneInfoMap)
	{
		const FName BoneName = Item.Key;
		const FBoneInfo& BoneInfo = Item.Value;
		if (BoneInfo.ParentBoneName != NAME_None)
		{
			FBoneInfo& ParentBoneInfo = BoneInfoMap[BoneInfo.ParentBoneName];
			ParentBoneInfo.Children.Add(BoneName);
		}
	}
	
	// For each joint in the remove list, mark it and all its children as to be removed.
	for (const FName BoneName: InBoneNames)
	{
		// We cannot remove the root bone.
		if (BoneName == RootBoneName)
		{
			continue;
		}
		
		TQueue<FName> BoneMarkingQueue;
		BoneMarkingQueue.Enqueue(BoneName);
		
		FName BoneToRemove;
		while (BoneMarkingQueue.Peek(BoneToRemove))
		{
			BoneMarkingQueue.Pop();

			if (FBoneInfo* BoneInfo = BoneInfoMap.Find(BoneToRemove); 
				BoneInfo != nullptr)
			{
				BoneInfo->bToRemove = true;
			
				for (const FName ChildBoneName: BoneInfo->Children)
				{
					BoneMarkingQueue.Enqueue(ChildBoneName);
				}
			}
		}
	}

	// Create a map that maps existing bone indexes to the non-removed bone indexes. All weights on bones
	// that are marked for removal will be moved to the closest ancestor bone not marked for removal.
	TMap<FBoneIndexType, FBoneIndexType> RemappedBoneIndices;
	
	for (const TPair<FName, FBoneInfo>& Item: BoneInfoMap)
	{
		const FBoneInfo* BoneInfo = &Item.Value;
		const FBoneID FromBoneID = BoneInfo->OriginalID;
		
		while (BoneInfo->bToRemove)
		{
			check(!BoneInfo->ParentBoneName.IsNone());
			BoneInfo = BoneInfoMap.Find(BoneInfo->ParentBoneName);
		}
		
		RemappedBoneIndices.Add(FromBoneID, BoneInfo->OriginalID);
	}
	
	TArray<FName> ProfileNames = Attributes.GetSkinWeightProfileNames();
	if (!InProfileNames.IsEmpty())
	{
		ProfileNames = ProfileNames.FilterByPredicate([&InProfileNames](const FName ProfileName) -> bool
			{
				return InProfileNames.Contains(ProfileName);
			});
	}
	
	// Loop over all the skin weight profiles. We can parallelize across profiles, but not for each VertexID, since the profile
	// data is stored as one giant dynamic array.
	ParallelFor(ProfileNames.Num(), [&InOutMeshDescription, &InBoneWeightSettings, &RemappedBoneIndices, &ProfileNames](int32 ProfileIndex)
		{
			using namespace UE::AnimationCore;
			
			const FName ProfileName = ProfileNames[ProfileIndex];
			FSkeletalMeshAttributes Attributes(InOutMeshDescription);
			FSkinWeightsVertexAttributesRef SkinWeights = Attributes.GetVertexSkinWeights(ProfileName);

			TMap<FBoneIndexType, uint16> BoneWeightMap;
			TArray<FBoneWeight> BoneWeights;
			
			for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
			{
				BoneWeightMap.Reset();
				BoneWeights.Reset();
				
				FVertexBoneWeights VertexBoneWeights = SkinWeights.Get(VertexID);
				
				for (const FBoneWeight& BoneWeight: VertexBoneWeights)
				{
					if (const FBoneIndexType* RemappedBoneIndex = RemappedBoneIndices.Find(BoneWeight.GetBoneIndex()))
					{
						// We can sum up the values here into an uint16 without risking an overflow, since
						// the weights are already normalized and will never exceed max uint16.
						BoneWeightMap.FindOrAdd(*RemappedBoneIndex) += BoneWeight.GetRawWeight();
					}
				}

				for (const TPair<FBoneIndexType, uint16>& BoneWeightPair : BoneWeightMap)
				{
					BoneWeights.Add(FBoneWeight(BoneWeightPair.Key, BoneWeightPair.Value));
				}
				
				SkinWeights.Set(VertexID, BoneWeights, InBoneWeightSettings);
			}
		});
}


void FSkeletalMeshOperations::RemoveUnusedBones(FMeshDescription& InOutMeshDescription)
{
	FSkeletalMeshAttributes Attributes(InOutMeshDescription);
	
	// No bone names to match against -- can't do any further processing.
	if (!Attributes.HasBones() || !Attributes.HasBoneNameAttribute())
	{
		return;
	}
	
	TArray<FName> ProfileNames = Attributes.GetSkinWeightProfileNames();
	
	TArray<TSet<FBoneIndexType>> BoneIndexSets;
	ParallelForWithTaskContext(BoneIndexSets, ProfileNames.Num(), [&InOutMeshDescription, &ProfileNames](TSet<FBoneIndexType>& BoneIndexes, int32 ProfileIndex)
		{
			using namespace UE::AnimationCore;
			
			const FName ProfileName = ProfileNames[ProfileIndex];
			FSkeletalMeshConstAttributes Attributes(InOutMeshDescription);
			FSkinWeightsVertexAttributesConstRef SkinWeights = Attributes.GetVertexSkinWeights(ProfileName);

			for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
			{
				FVertexBoneWeightsConst VertexBoneWeights = SkinWeights.Get(VertexID);
				
				for (const FBoneWeight& BoneWeight: VertexBoneWeights)
				{
					BoneIndexes.Add(BoneWeight.GetBoneIndex());
				}
			}
		});
	
	TSet<FBoneIndexType> UsedBoneIndexes;
	for (const TSet<FBoneIndexType>& BoneIndexes: BoneIndexSets)
	{
		UsedBoneIndexes.Append(BoneIndexes);
	}
	
	// Validate the indexes.
	for (const FBoneIndexType BoneIndex: UsedBoneIndexes)
	{
		if (!Attributes.Bones().IsValid(FBoneID(BoneIndex)))
		{
			UE_LOGF(LogSkeletalMeshOperations, Error, "Found invalid bone index %d (only %d bones defined)", BoneIndex, Attributes.Bones().Num());
			return;
		}
	}
	
	// Ensure that all ancestors of used bones are in the used list as well.
	FSkeletalMeshAttributes::FBoneNameAttributesConstRef BoneNameAttr = Attributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndexAttr = Attributes.GetBoneParentIndices();
	TSet<FBoneID> ParentsFound;
	TSet<FBoneIndexType> UsedParentBoneIndexes;
	int32 RootBoneIndex = INDEX_NONE;
	for (const FBoneIndexType BoneIndex: UsedBoneIndexes)
	{
		FBoneID BoneID(BoneIndex);
		ParentsFound.Reset();
		
		for (;;)
		{
			FBoneID ParentBoneID = FBoneID(BoneParentIndexAttr.Get(BoneID));
			if (ParentsFound.Contains(ParentBoneID))
			{
				UE_LOGF(LogSkeletalMeshOperations, Error, "Bone hierarchy is cyclical (with bone '%ls' [%d])", 
					*BoneNameAttr.Get(ParentBoneID).ToString(), ParentBoneID.GetValue());
				return;
			}
			
			if (ParentBoneID.GetValue() == INDEX_NONE)
			{
				if (RootBoneIndex != INDEX_NONE && BoneID.GetValue() != RootBoneIndex)
				{
					UE_LOGF(LogSkeletalMeshOperations, Error, "Found a duplicate root bone %d (first found root %d)", BoneID.GetValue(), RootBoneIndex);
					return;
				}
				
				// Found a root bone.
				RootBoneIndex = BoneID.GetValue();
				break;
			}
			if (!Attributes.Bones().IsValid(ParentBoneID))
			{
				UE_LOGF(LogSkeletalMeshOperations, Error, "Found invalid bone index %d (only %d bones defined)", ParentBoneID.GetValue(), Attributes.Bones().Num());
				return;
			}
			
			BoneID = ParentBoneID;
			UsedParentBoneIndexes.Add(BoneID.GetValue());
			ParentsFound.Add(BoneID.GetValue());
		}
	}
	
	UsedBoneIndexes.Append(UsedParentBoneIndexes);
	
	// Create a mapping to go from sparse indices to compact indices.
	TArray<FBoneIndexType> BoneIndexMapping;
	FBoneIndexType CurrentIndex = 0;
	
	// We need to loop over all bone attribute values, not just those that are defined as IDs, since
	// IDs can be non-contiguous and the indices in the bones refer to the bone ID. So the mapping
	// array needs to be able to handle all possible bone indices.
	for (int32 BoneIndex = 0; BoneIndex < Attributes.Bones().GetArraySize(); ++BoneIndex)
	{
		const FBoneID BoneID(BoneIndex);

		if (UsedBoneIndexes.Contains(BoneID.GetValue()))
		{
			BoneIndexMapping.Add(CurrentIndex++);
		}
		else
		{
			BoneIndexMapping.Add(INDEX_NONE);
		}
	}
	
	// Remap all existing bone bindings to point to the new, compacted bone IDs.
	ParallelFor(ProfileNames.Num(), [&InOutMeshDescription, &Attributes, &BoneIndexMapping, &ProfileNames](int32 ProfileIndex)
		{
			using namespace UE::AnimationCore;
			
			const FName ProfileName = ProfileNames[ProfileIndex];
			FSkinWeightsVertexAttributesRef SkinWeights = Attributes.GetVertexSkinWeights(ProfileName);
			TArray<FBoneWeight> BoneWeights;
			BoneWeights.Reserve(MaxInlineBoneWeightCount);

			for (FVertexID VertexID: InOutMeshDescription.Vertices().GetElementIDs())
			{
				FVertexBoneWeights VertexBoneWeights = SkinWeights.Get(VertexID);
				BoneWeights.Reset();
				for (FBoneWeight BoneWeight: VertexBoneWeights)
				{
					BoneWeight.SetBoneIndex(BoneIndexMapping[BoneWeight.GetBoneIndex()]);
					BoneWeights.Add(BoneWeight);
				}
				
				SkinWeights.SetRaw(VertexID, BoneWeights);
			}
		});
	
	// Delete the unused IDs.
	for (const FBoneID BoneID: Attributes.Bones().GetElementIDs())
	{
		if (UsedBoneIndexes.Contains(BoneID.GetValue()))
		{
			const int32 ParentBoneIndex = BoneParentIndexAttr.Get(BoneID);
			
			// Only remap if we have a valid remapping. Leave the index as-is otherwise (it could be indicative
			// of a root bone, or just a very broken index).
			if (BoneIndexMapping.IsValidIndex(ParentBoneIndex) && BoneIndexMapping[ParentBoneIndex] != INDEX_NONE)
			{
				BoneParentIndexAttr.Set(BoneID, BoneIndexMapping[ParentBoneIndex]);
			}
		}
		else
		{
			// Not in the used set. Delete the bone now. We can use IDs here since deletion does not alter
			// the value of the IDs, instead it leaves gaps in the data. Hence, the compaction below.
			Attributes.Bones().Remove(BoneID);
		}
	}
	
	// Compact the bone attribute values to make sure the IDs of the bones are contiguous and that the
	// indices in the parent bone attribute are correct.
	TSparseArray<int32> CompactionRemapping;
	Attributes.Bones().Compact(CompactionRemapping);
}

#undef LOCTEXT_NAMESPACE
