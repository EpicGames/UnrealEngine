// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SkeletalMeshElementTypes.h"

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "MeshAttributeArray.h"
#include "SkinWeightsAttributesRef.h"
#include "StaticMeshAttributes.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

// Forward declarations
template <typename AttributeType> class TArrayAttribute;
struct FMeshDescription;

// Add any skeletalmesh specific attributes here
namespace MeshAttribute
{
 	namespace Vertex
 	{
 		// Name of the default skin weights attribute.
		extern SKELETALMESHDESCRIPTION_API const FName SkinWeights;

 		extern SKELETALMESHDESCRIPTION_API const FName ImportPointIndex;
 	}

	namespace PolygonGroup
 	{
 		extern SKELETALMESHDESCRIPTION_API const FName ParentPolygonGroup;
 	}

	namespace Bone
	{
		extern SKELETALMESHDESCRIPTION_API const FName Name;
		extern SKELETALMESHDESCRIPTION_API const FName ParentIndex;
		extern SKELETALMESHDESCRIPTION_API const FName Pose;
		extern SKELETALMESHDESCRIPTION_API const FName Color;
	}

	namespace SourceGeometryPart
 	{
 		extern SKELETALMESHDESCRIPTION_API const FName Name;
 		extern SKELETALMESHDESCRIPTION_API const FName VertexOffsetAndCount;
 	}
}


class FSkeletalMeshAttributesShared
{
public:
	using FBoneArray = TMeshElementContainer<FBoneID>;
	
	using FBoneNameAttributesRef = TMeshAttributesRef<FBoneID, FName>;
	using FBoneNameAttributesConstRef = TMeshAttributesConstRef<FBoneID, FName>;
	
	using FBoneParentIndexAttributesRef = TMeshAttributesRef<FBoneID, int32>;
	using FBoneParentIndexAttributesConstRef = TMeshAttributesConstRef<FBoneID, int32>;
	
	using FBonePoseAttributesRef = TMeshAttributesRef<FBoneID, FTransform>;
	using FBonePoseAttributesConstRef = TMeshAttributesConstRef<FBoneID, FTransform>;
	
	using FBoneColorAttributesRef = TMeshAttributesRef<FBoneID, FVector4f>;
	using FBoneColorAttributesConstRef = TMeshAttributesConstRef<FBoneID, FVector4f>;

	using FSourceGeometryPartArray = TMeshElementContainer<FSourceGeometryPartID>;
	
	using FSourceGeometryPartNameRef = TMeshAttributesRef<FSourceGeometryPartID, FName>;
	using FSourceGeometryPartNameConstRef = TMeshAttributesConstRef<FSourceGeometryPartID, FName>;
	
	using FSourceGeometryPartVertexOffsetAndCountRef = TMeshAttributesRef<FSourceGeometryPartID, TArrayView<int32>>;
	using FSourceGeometryPartVertexOffsetAndCountConstRef = TMeshAttributesConstRef<FSourceGeometryPartID, TArrayView<int32>>;
	
	/** 
	 * Name of the mesh element type representing bones.
	 * 
	 * @note this is different from the MeshAttribute::Bone::Name attribute. This is a name of the element that is 
	 * added to the  mesh description to represent Bones (similar to the Vertex/Polygons/Edges elements). 
	 * MeshAttribute::Bone::Name is just one of the attributes of the Bones element.
	 */
	static SKELETALMESHDESCRIPTION_API FName BonesElementName;

	
	/**
	 * Name of the mesh element representing import data for meshes that were imported from separate geometries
	 * into a single skeletal mesh piece. Can be used to match parts of the mesh back to the constituent parts
	 * of the import source.
	 */
	static SKELETALMESHDESCRIPTION_API FName SourceGeometryPartElementName;
	
	
	// The name of the default skin weight profile.
	static SKELETALMESHDESCRIPTION_API FName DefaultSkinWeightProfileName;
	
	SKELETALMESHDESCRIPTION_API FSkeletalMeshAttributesShared(const FMeshDescription& InMeshDescription);

	/**
	 * Returns the parent polygon group read-only attribute.
	 */
	TPolygonGroupAttributesConstRef<int32> GetPolygonGroupParents() const { return MeshDescriptionShared.PolygonGroupAttributes().GetAttributesRef<int32>(MeshAttribute::PolygonGroup::ParentPolygonGroup); }
	
	//
	// Skin Weights Methods
	//

	/** Returns the list of all registered skin weight profile names on this mesh.
	 *  \param bInUserDefinedOnly Only return user-defined profiles, not the default profile.
	 *  \return The list of the profile names.
	 */
	SKELETALMESHDESCRIPTION_API TArray<FName> GetSkinWeightProfileNames(const bool bInUserDefinedOnly = false) const;
	
	/// Returns \c true if the given identifier is a valid profile name. If the name is empty, or matches the default profile,
	/// then the profile name is considered invalid. 
	static SKELETALMESHDESCRIPTION_API bool IsValidSkinWeightProfileName(const FName InProfileName);

	/// Helper function that indicates whether an attribute name represents a skin weight attribute.
	static SKELETALMESHDESCRIPTION_API bool IsSkinWeightAttribute(const FName InAttributeName);

	/// Returns a skin profile name from the attribute name, if the attribute name is a valid skin weights
	/// attribute.
	static SKELETALMESHDESCRIPTION_API FName GetProfileNameFromAttribute(const FName InAttributeName);

	SKELETALMESHDESCRIPTION_API FSkinWeightsVertexAttributesConstRef GetVertexSkinWeights(const FName InProfileName = NAME_None) const;
	
	SKELETALMESHDESCRIPTION_API FSkinWeightsVertexAttributesConstRef GetVertexSkinWeightsFromAttributeName(const FName InAttributeName = NAME_None) const;

	//
	// Morph Target Methods
	//

	/// Returns the list of all registered morph targets on this mesh.
	SKELETALMESHDESCRIPTION_API TArray<FName> GetMorphTargetNames() const;
	
	/// Returns \c true if the given attribute name refers to a morph target attribute.
	static SKELETALMESHDESCRIPTION_API bool IsMorphTargetAttribute(const FName InAttributeName);

	/// Returns the name of a morph target given the attribute name. If the given attribute name is invalid, \c NAME_None is returned.
	static SKELETALMESHDESCRIPTION_API FName GetMorphTargetNameFromAttribute(const FName InAttributeName);

	/// Returns a read-only vertex position delta attribute for the given morph target. The delta is the difference between the point position of the
	/// mesh and the point position of the morphed mesh. Adding this value to the point position of the mesh will return the morph point position.
	SKELETALMESHDESCRIPTION_API TVertexAttributesConstRef<FVector3f> GetVertexMorphPositionDelta(const FName InMorphTargetName) const;

	/// Returns a read-only vertex instance normal delta attribute for the given morph target. The delta is the difference between the normal vector 
	/// of the mesh and the normal vector of the desired morphed mesh. Adding this value to the normal vector of the mesh will return the morph's 
	/// normal vector for this vertex instance.
	/// If the morph was not registered to include normals, this will return an invalid attribute. 
	SKELETALMESHDESCRIPTION_API TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceMorphNormalDelta(const FName InMorphTargetName) const;

	SKELETALMESHDESCRIPTION_API bool HasMorphTargetPositionsAttribute(const FName InMorphTargetName) const;

	SKELETALMESHDESCRIPTION_API bool HasMorphTargetNormalsAttribute(const FName InMorphTargetName) const;
	
	//
	// Bones Methods
	//

	SKELETALMESHDESCRIPTION_API bool HasBoneColorAttribute() const;

	SKELETALMESHDESCRIPTION_API bool HasBoneNameAttribute() const;

	SKELETALMESHDESCRIPTION_API bool HasBonePoseAttribute() const;

	SKELETALMESHDESCRIPTION_API bool HasBoneParentIndexAttribute() const;

	SKELETALMESHDESCRIPTION_API const FBoneArray& Bones() const;

	SKELETALMESHDESCRIPTION_API const TAttributesSet<FBoneID>& BoneAttributes() const;

	/**  @return true, if bone element was added to the MeshDescription. */
	bool HasBones() const
	{
		return BoneElementsShared != nullptr;
	}

	/**  @return the number of the bones. 0 if bone element does not exist in the FMeshDescription. */
	SKELETALMESHDESCRIPTION_API int32 GetNumBones() const;

	/** @return true, if the passed bone ID is valid */
	SKELETALMESHDESCRIPTION_API bool IsBoneValid(const FBoneID BoneID) const;

	SKELETALMESHDESCRIPTION_API FBoneNameAttributesConstRef GetBoneNames() const;

	SKELETALMESHDESCRIPTION_API FBoneParentIndexAttributesConstRef GetBoneParentIndices() const;

	SKELETALMESHDESCRIPTION_API FBonePoseAttributesConstRef GetBonePoses() const;

	SKELETALMESHDESCRIPTION_API FBoneColorAttributesConstRef GetBoneColors() const;


	//
	bool HasSourceGeometryParts() const
	{
		return SourceGeometryPartElementsShared != nullptr;
	}

	SKELETALMESHDESCRIPTION_API const FSourceGeometryPartArray& SourceGeometryParts() const;
	
	SKELETALMESHDESCRIPTION_API const TAttributesSet<FSourceGeometryPartID>& SourceGeometryPartAttributes() const;
		
	SKELETALMESHDESCRIPTION_API int32 GetNumSourceGeometryParts() const;

	SKELETALMESHDESCRIPTION_API bool IsSourceGeometryPartValid(const FSourceGeometryPartID InSourceGeometryPartID) const;

	SKELETALMESHDESCRIPTION_API FSourceGeometryPartNameConstRef GetSourceGeometryPartNames() const;
	
	SKELETALMESHDESCRIPTION_API FSourceGeometryPartVertexOffsetAndCountConstRef GetSourceGeometryPartVertexOffsetAndCounts() const;
	
protected:
	/// Construct a name for a skin weight attribute with the given skin weight profile name.
	/// Each mesh description can hold different skin weight profiles, although the default
	/// is always present.
	static SKELETALMESHDESCRIPTION_API FName CreateSkinWeightAttributeName(const FName InProfileName);

	/// Construct a name for a morph target attribute with the given a user-visible morph target name.
	static SKELETALMESHDESCRIPTION_API FName CreateMorphTargetAttributeName(const FName InMorphTargetName);
	
	const FMeshElementChannels* BoneElementsShared = nullptr;
	const FMeshElementChannels* SourceGeometryPartElementsShared = nullptr;

private:
	const FMeshDescription& MeshDescriptionShared;
};


class FSkeletalMeshAttributes :
	public FStaticMeshAttributes,
	public FSkeletalMeshAttributesShared
{
public:

	SKELETALMESHDESCRIPTION_API explicit FSkeletalMeshAttributes(FMeshDescription& InMeshDescription);

	SKELETALMESHDESCRIPTION_API virtual void Register(bool bKeepExistingAttribute = false) override;

	/** Returns \c true if a given attribute name is a name for a reserved attribute or not. If not a reserved
	 *  attribute, it's one that has been user-defined and is not required by any system.
	 */
	static bool IsReservedAttributeName(const FName InAttributeName)
	{
		return FStaticMeshAttributes::IsReservedAttributeName(InAttributeName) ||
			   IsSkinWeightAttribute(InAttributeName) ||
			   IsMorphTargetAttribute(InAttributeName) ||
			   InAttributeName == MeshAttribute::Vertex::ImportPointIndex ||
			   InAttributeName == MeshAttribute::PolygonGroup::ParentPolygonGroup ||
			   InAttributeName == MeshAttribute::Bone::Name ||
			   InAttributeName == MeshAttribute::Bone::ParentIndex ||
			   InAttributeName == MeshAttribute::Bone::Pose ||
			   InAttributeName == MeshAttribute::Bone::Color;
	}
	
	//
	/** Register an attribute that can be used to maintain a relationship between a given vertex and the point it originated from
	 *  on the original imported mesh. This can be useful when trying to match the stored mesh to the import mesh as stored in the import file
	 *  (e.g. FBX, Alembic, glTF, etc).
	 */
	SKELETALMESHDESCRIPTION_API bool RegisterImportPointIndexAttribute();

	/** Convenience function to unregister the attribute that maintains the relationship between a given vertex and the originating point
	 *  on the import mesh. Use if updating the mesh description but no longer maintaining this relationship (e.g. the topology changed and
	 *  so this map cannot possibly be accurate).
	 */
	SKELETALMESHDESCRIPTION_API void UnregisterImportPointIndexAttribute();

	/**
	 * Register an attribute on the polygon group that can be used for polygon groups that have been split to indicate which
	 * group they got split off from.
	 * @see FSkeletalMeshOperations::PartitionByMaximumUniqueBoneIndexCount
	 */
	SKELETALMESHDESCRIPTION_API bool RegisterPolygonGroupParentAttribute();

	/**
	 * Returns the parent polygon group attribute.
	 */
	TPolygonGroupAttributesRef<int32> GetPolygonGroupParents() { return MeshDescription.PolygonGroupAttributes().GetAttributesRef<int32>(MeshAttribute::PolygonGroup::ParentPolygonGroup); }
	
	//
	// Skin Weights Methods
	//

	/// Register a new skin weight profile with the given name. The attribute name will encode the profile name and
	/// it will be listed in GetSkinWeightProfileNames(). Returns \c true if the profile was successfully registered.
	/// Returns \c false if the attribute was already registered or if IsValidSkinWeightProfileName() returned false.
	SKELETALMESHDESCRIPTION_API bool RegisterSkinWeightAttribute(const FName InProfileName);

	/// Unregister an existing skin weight profile with the given name
	/// Returns \c true if theprofile was successfully unregistered.
	/// Returns \c false if the attribute wasn't registered or if \c InProfileName is empty.
	SKELETALMESHDESCRIPTION_API bool UnregisterSkinWeightAttribute(const FName InProfileName);

	/// Returns the skin weight profile given by its name. NAME_None corresponds to the default profile.
	SKELETALMESHDESCRIPTION_API FSkinWeightsVertexAttributesRef GetVertexSkinWeights(const FName InProfileName = NAME_None);

	SKELETALMESHDESCRIPTION_API FSkinWeightsVertexAttributesRef GetVertexSkinWeightsFromAttributeName(const FName InAttributeName = NAME_None);

	//
	// Morph Target Methods
	//
	
	/// Register a new morph target with the given name. The attribute name will encode the user-defined morph target name and
	/// it will be listed in GetMorphTargetNames(). Returns \c true if the morph target was successfully registered.
	/// Returns \c false if the attribute was already registered or if \c InMorphTargetName is empty.
	/// The position delta is stored as a vertex attributes, and the optional normal is stored as a vertex instance attribute to
	/// allow for hard edges on morph targets.
	/// \param InMorphTargetName The name of the morph target to add. Cannot be empty.
	/// \param bIncludeNormals Set to \c true if per-vertex instance normals (Z-tangent) should be included. Otherwise the normals
	///		will be automatically computed and only the position delta vertex attribute registered.
	SKELETALMESHDESCRIPTION_API bool RegisterMorphTargetAttribute(const FName InMorphTargetName, const bool bIncludeNormals);

	/// Unregister an existing morph target with the given name (as returned by GetMorphTargetNames()). Returns \c true if the morph target
	/// was successfully unregistered.
	/// Returns \c false if the attribute wasn't registered or if \c InMorphTargetName is empty.
	SKELETALMESHDESCRIPTION_API bool UnregisterMorphTargetAttribute(const FName InMorphTargetName);

	/// Returns a writable vertex position delta attribute for the given morph target. The delta is the difference between the point position of the
	/// mesh and the point position of the morphed mesh. Adding this value to the point position of the mesh will return the morph point position.
	SKELETALMESHDESCRIPTION_API TVertexAttributesRef<FVector3f> GetVertexMorphPositionDelta(const FName InMorphTargetName);

	/// Returns a writable vertex instance normal delta attribute for the given morph target. The delta is the difference between the normal vector 
	/// of the mesh and the normal vector of the desired morphed mesh. Adding this value to the normal vector of the mesh will return the morph's 
	/// normal vector for this vertex instance.
	/// If the morph was not registered to include normals, this will return an invalid attribute. 
	SKELETALMESHDESCRIPTION_API TVertexInstanceAttributesRef<FVector3f> GetVertexInstanceMorphNormalDelta(const FName InMorphTargetName);
	
	//
	// Bones Methods
	//

	/** Register an optional color attribute for bones */
	SKELETALMESHDESCRIPTION_API void RegisterColorAttribute();

	SKELETALMESHDESCRIPTION_API FBoneArray& Bones();

	SKELETALMESHDESCRIPTION_API TAttributesSet<FBoneID>& BoneAttributes();

	/** Reserves space for this number of new bones */
	SKELETALMESHDESCRIPTION_API void ReserveNewBones(const int InNumBones);

	/** Adds a new bone and returns its ID */
	SKELETALMESHDESCRIPTION_API FBoneID CreateBone();

	/** Adds a new bone  with the given ID */
	SKELETALMESHDESCRIPTION_API void CreateBone(const FBoneID BoneID);

	/** Deletes a bone with the given ID */
	SKELETALMESHDESCRIPTION_API void DeleteBone(const FBoneID BoneID);

	SKELETALMESHDESCRIPTION_API FBoneNameAttributesRef GetBoneNames();
	
	SKELETALMESHDESCRIPTION_API FBoneParentIndexAttributesRef GetBoneParentIndices();

	SKELETALMESHDESCRIPTION_API FBonePoseAttributesRef GetBonePoses();

	SKELETALMESHDESCRIPTION_API FBoneColorAttributesRef GetBoneColors();


	/* Source Geometry Parts */

	/** This will register the source geometry attributes and their element container on the mesh description so that
	 * source geometry part information can be stored.
	 */
	SKELETALMESHDESCRIPTION_API void RegisterSourceGeometryPartsAttributes();

	SKELETALMESHDESCRIPTION_API FSourceGeometryPartArray& SourceGeometryParts();
	
	SKELETALMESHDESCRIPTION_API TAttributesSet<FSourceGeometryPartID>& SourceGeometryPartAttributes();
	
	SKELETALMESHDESCRIPTION_API FSourceGeometryPartID CreateSourceGeometryPart();

	SKELETALMESHDESCRIPTION_API void DeleteSourceGeometryPart(FSourceGeometryPartID InSourceGeometryPartID);

	SKELETALMESHDESCRIPTION_API FSourceGeometryPartNameRef GetSourceGeometryPartNames();
	
	SKELETALMESHDESCRIPTION_API FSourceGeometryPartVertexOffsetAndCountRef GetSourceGeometryPartVertexOffsetAndCounts();
	
	
	
private:
	FMeshElementChannels* BoneElements = nullptr;
	FMeshElementChannels* SourceGeometryPartElements = nullptr;
};


class FSkeletalMeshConstAttributes :
	public FStaticMeshConstAttributes,
	public FSkeletalMeshAttributesShared
{
public:

	explicit FSkeletalMeshConstAttributes(const FMeshDescription& InMeshDescription) :
		FStaticMeshConstAttributes(InMeshDescription),
		FSkeletalMeshAttributesShared(InMeshDescription)
	{}

};
