// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDescriptionBase.h"

#include "SkeletalMeshAttributes.h" 

#include "SkeletalMeshDescription.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class USkeletalMeshDescription : public UMeshDescriptionBase
{
public:
	GENERATED_BODY()

	SKELETALMESHDESCRIPTION_API virtual void RegisterAttributes() override;

	virtual FSkeletalMeshAttributes& GetRequiredAttributes() override
	{ 
		return static_cast<FSkeletalMeshAttributes&>(*RequiredAttributes);
	}

	virtual const FSkeletalMeshAttributes& GetRequiredAttributes() const override
	{
		return static_cast<const FSkeletalMeshAttributes&>(*RequiredAttributes);
	}
	
};
