// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshTypes.h"

#include "SkeletalMeshElementTypes.generated.h"

USTRUCT(BlueprintType)
struct FBoneID : public FElementID
{
	GENERATED_BODY()

	FBoneID()
	{
	}

	FBoneID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	FBoneID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	friend uint32 GetTypeHash( const FBoneID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}
};

USTRUCT(BlueprintType)
struct FSourceGeometryPartID : public FElementID
{
	GENERATED_BODY()

	FSourceGeometryPartID() = default;

	explicit FSourceGeometryPartID( const FElementID InitElementID )
		: FElementID( InitElementID.GetValue() )
	{
	}

	explicit FSourceGeometryPartID( const int32 InitIDValue )
		: FElementID( InitIDValue )
	{
	}

	friend uint32 GetTypeHash( const FSourceGeometryPartID& Other )
	{
		return GetTypeHash( Other.IDValue );
	}
};
