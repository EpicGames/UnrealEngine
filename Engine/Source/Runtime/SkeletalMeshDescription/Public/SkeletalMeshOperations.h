// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "BoneWeights.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "MeshTypes.h"
#include "StaticMeshOperations.h"

struct FMeshDescription;


DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshOperations, Log, All);



class FSkeletalMeshOperations : public FStaticMeshOperations
{
public:
	struct FSkeletalMeshAppendSettings
	{
		FSkeletalMeshAppendSettings()
			: SourceVertexIDOffset(0)
		{}

		int32 SourceVertexIDOffset;
		TArray<FBoneIndexType> SourceRemapBoneIndex;
		bool bAppendVertexAttributes = false;
	};
	
	static SKELETALMESHDESCRIPTION_API void AppendSkinWeight(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, FSkeletalMeshAppendSettings& AppendSettings);

	/** Copies skin weight attribute from one mesh to another. Assumes the two geometries are identical or near-identical.
	 *  Uses closest triangle on the target mesh to interpolate skin weights to each of the points on the target mesh.
	 *  Attributes for the given profiles on both meshes should exist in order for this function to succeed. 
	 *  @param InSourceMesh The mesh to copy skin weights from.
	 *  @param InTargetMesh The mesh to copy skin weights to.
	 *  @param InSourceProfile The name of the skin weight profile on the source mesh to read from.
	 *  @param InTargetProfile The name of the skin weight profile on the target mesh to write to.
	 *  @param SourceBoneIndexToTargetBoneIndexMap An optional mapping table to map bone indexes on the source mesh to the target mesh.
	 *     The table needs to be complete for all the source bone indexes to valid target bone indexes, otherwise the behavior
	 *     is undefined. If the table is not given, the bone indexes on the source and target meshes are assumed to be the same.
	 */
	static SKELETALMESHDESCRIPTION_API bool CopySkinWeightAttributeFromMesh(
		const FMeshDescription& InSourceMesh,
		FMeshDescription& InTargetMesh,
		const FName InSourceProfile,
		const FName InTargetProfile,
		const TMap<int32, int32>* SourceBoneIndexToTargetBoneIndexMap
		);

	/** Remaps the bone indices on all skin weight attributes from one index to another. The array view should contain a full mapping of all the
	 *  bone indices contained in the skin weights. The array is indexed by the current bone index and the value at that index is the new bone index.
	 *  If the mapping is incomplete or if two entries map to the same bone, the result is undefined. No prior checking is performed.
	 *  @param InMesh The mesh on which to modify all skin weight attributes to remap their bone indices.
	 *  @param InBoneIndexMapping The mapping from one bone index to another. The old bone index is used to index into the array, the value at that
	 *     position is the new bone index.
	 *  @return true if the operation was successful. If the mapping array was incomplete then this will return false. If there are no skin weight
	 *     attributes on the mesh, then the operation is still deemed successful.
	 */  
	static SKELETALMESHDESCRIPTION_API bool RemapBoneIndicesOnSkinWeightAttribute(
		FMeshDescription& InMesh,
		TConstArrayView<int32> InBoneIndexMapping
		);

	/** Returns a mesh in the pose given by the component-space transforms passed in. The list of transforms should match exactly
	 *  the list of bones stored on the mesh. If not, the function fails and returns \c false.
	 *  If there are no skin weights on the mesh, or the named skin weight profile doesn't exist, the function also returns \c false.
	 *  The resulting bones on the mesh will have their bone-space transforms updated so that the same mesh can be re-posed as needed.
	 *  @param InSourceMesh The mesh to deform.
	 *  @param OutTargetMesh The deformed mesh result.
	 *  @param InComponentSpaceTransforms The component space transforms used to move the mesh joints for deforming, using linear-blend skinning.
	 *  @param InSkinWeightProfile The skin weight profile to use as the source of skin weights for the deformation.
	 *  @param InMorphTargetWeights Optional morph target weights to apply. Any morph target that doesn't exist is ignored.
	 *  @return \c true if the operation succeeded.
	 */
	static SKELETALMESHDESCRIPTION_API bool GetPosedMesh(
		const FMeshDescription& InSourceMesh,
		FMeshDescription& OutTargetMesh,
		TConstArrayView<FTransform> InComponentSpaceTransforms,
		const FName InSkinWeightProfile = NAME_None,
		const TMap<FName, float>& InMorphTargetWeights = {}
		);

	/** Returns a mesh in the pose given by the bone-space transforms passed in. The transforms simply replace the matching ref pose transforms
	 *  stored in the bone data on the mesh. Any named transform, that does not match a bone on the mesh, is ignored.  
	 *  If there are no skin weights on the mesh, or the named skin weight profile doesn't exist, the function also returns \c false.
	 *  The resulting bones on the mesh will have their bone-space transforms updated so that the same mesh can be re-posed as needed.
	 *  
	 *  @param InSourceMesh The mesh to deform.
	 *  @param OutTargetMesh The deformed mesh result.
	 *  @param InBoneSpaceTransforms A map of named bone-space transforms.
	 *  @param InSkinWeightProfile The skin weight profile to use as the source of skin weights for the deformation.
	 *  @param InMorphTargetWeights Optional morph target weights to apply. Any morph target that doesn't exist is ignored.
	 *  @return \c true if the operation succeeded.
	 */
	static SKELETALMESHDESCRIPTION_API bool GetPosedMesh(
		const FMeshDescription& InSourceMesh,
		FMeshDescription& OutTargetMesh,
		const TMap<FName, FTransform>& InBoneSpaceTransforms, 
		const FName InSkinWeightProfile = NAME_None,
		const TMap<FName, float>& InMorphTargetWeights = {}
		);

	/** Returns a mesh in the pose given by the component-space transforms passed in. The list of transforms should match exactly
	 *  the list of bones stored on the mesh. If not, the function fails and returns \c false.
	 *  If there are no skin weights on the mesh, or the named skin weight profile doesn't exist, the function also returns \c false.
	 *  if bInWriteBonePose is true, the resulting bones on the mesh will have their bone-space transforms updated so that the same mesh can be re-posed as needed.
	 *  @param InOutTargetMesh rest pose mesh to be deformed in place
	 *  @param InComponentSpaceTransforms The component space transforms used to move the mesh joints for deforming, using linear-blend skinning.
	 *  @param InSkinWeightProfile The skin weight profile to use as the source of skin weights for the deformation.
	 *  @param InMorphTargetWeights Optional morph target weights to apply. Any morph target that doesn't exist is ignored.
	 *  @param bInSkipRecomputeNormalsTangents skip recompute normals and tangents to improve performance
	 *  @param bInWriteBonePose whether to write the current bone pose into the mesh description.
	 *  @return \c true if the operation succeeded.
	 */
	static SKELETALMESHDESCRIPTION_API bool GetPosedMeshInPlace(
		FMeshDescription& InOutTargetMesh,
		TConstArrayView<FTransform> InComponentSpaceTransforms,
		const FName InSkinWeightProfile = NAME_None,
		const TMap<FName, float>& InMorphTargetWeights = {},
		bool bInSkipRecomputeNormalsTangents = false,
		bool bInWriteBonePose = false
		);

	/** Returns the unposed version of the provided posed mesh
	 *  
	 *  @param InPosedMesh The posed mesh,
	 *  @param InRefMesh Ref mesh containing morph target deltas of existing morphs
	 *  @param RefBoneTransforms Ref component space bone transforms
	 *  @param OutUnposedMesh The mesh with pose reset to reference
	 *  @param InComponentSpaceTransforms Current component space bone transforms
	 *  @param InSkinWeightProfile The skin weight profile to use as the source of skin weights for the deformation.
	 *  @param InMorphTargetWeights Active morph target weights producing the fully deformed mesh. Don't include the weight of the morph target you want to extract
	 *  @return \c true if the operation succeeded.
	 */
	static SKELETALMESHDESCRIPTION_API bool GetUnposedMesh(
		const FMeshDescription& InPosedMesh,
		const FMeshDescription& InRefMesh,
		TArray<FTransform>& RefBoneTransforms,
		FMeshDescription& OutUnposedMesh,
		TConstArrayView<FTransform> InComponentSpaceTransforms,
		const FName InSkinWeightProfile,
		const TMap<FName, float>& InMorphTargetWeights
		);

	/** Returns the unposed version of the provided posed mesh
	 *  
	 *  @param InOutTargetMesh input is the posed mesh, output is the mesh reset to ref pose
	 *  @param InRefMesh Ref mesh containing morph target deltas of existing morphs
	 *  @param RefBoneTransforms Ref component space bone transforms
	 *  @param InComponentSpaceTransforms Current component space bone transforms
	 *  @param InSkinWeightProfile The skin weight profile to use as the source of skin weights for the deformation.
	 *  @param InMorphTargetWeights Active morph target weights producing the fully deformed mesh. Don't include the weight of the morph target you want to extract
	 *  @return \c true if the operation succeeded.
	 */
	static SKELETALMESHDESCRIPTION_API bool GetUnposedMeshInPlace(
		FMeshDescription& InOutTargetMesh,
		const FMeshDescription& InRefMesh,
		TArray<FTransform>& RefBoneTransforms,
		TConstArrayView<FTransform> InComponentSpaceTransforms,
		const FName InSkinWeightProfile,
		const TMap<FName, float>& InMorphTargetWeights,
		bool bInWriteBonePose = false
		);

	/**
	 * A simpler variant of FStaticMeshOperations::ConvertHardEdgesToSmoothGroup that assumes that hard edges always form closed regions. 
	 */
	static SKELETALMESHDESCRIPTION_API void ConvertHardEdgesToSmoothMasks(const FMeshDescription& InMeshDescription, TArray<uint32>& OutSmoothMasks);

	/**
	 * Helper function to follow behavior generated in FSkeletalMeshImportData::GetMeshDescription, just straight on FMeshDescriptions.
	 * FSkeletalMeshImportData::GetMeshDescription makes the VertexInstances a single usage semantics.
	 * It also re-orders the vertexInstances to be in a straight increasing sequence, which seems to affect the normal and tangent (FSkeletalMeshOperations::Compute..) generation.
	 *		(Which means, we can't do the restructuring in place.)
	 * For that reason we re-create the MeshDescription into the TargetMeshDescription which is expected to be empty.
	 */
	static SKELETALMESHDESCRIPTION_API void FixVertexInstanceStructure(FMeshDescription& SourceMeshDescription, FMeshDescription& TargetMeshDescription /*Expected to be empty*/,
		const TArray<uint32>& SourceSmoothingMasks, TArray<uint32>& TargetFaceSmoothingMasks);

	/**
	 * Helper function to follow behavior generated in FSkeletalMeshImportData::GetMeshDescription, just straight on FMeshDescriptions
	 * It will do the following steps:
	 *		- ConvertSmoothGroupToHardEdges
	 *		- ValidateAndFixData
	 *		- HasInvalidVertexInstanceNormalsOrTangents -> ComputeTriangleTangentsAndNormals
	 *		- (Re)BuildIndexers
	 */ 
	static SKELETALMESHDESCRIPTION_API void ValidateFixComputeMeshDescriptionData(FMeshDescription& MeshDescription, const TArray<uint32>& FaceSmoothingMasks, int32 LODIndex, const bool bComputeWeightedNormals, const FString& SkeletalMeshPath);

	/**
	 * Function will do the following steps on the MeshDescription (SkeletalMeshDescription) :
	 * - Sort influences by weight and BoneIndex.
	 * - Normalize influence weights.
	 * - provide flag if InfluenceCount exceeds MAX_TOTAL_INFLUENCES
	 * - Make sure all verts have influences set (if none exist bone 0 with weight 1)
	 * 
	 */
	static SKELETALMESHDESCRIPTION_API void ValidateAndFixInfluences(FMeshDescription& MeshDescription, bool& bOutInfluenceCountLimitHit);

	/**
	 * Applies the Rig / Skinning found in RigMeshDescription to the Geometry found in GeomeshDescription
	 * Desired behavior of this function was targeting FSkeletalMeshImportData::ApplyRigToGeo.
	 * Important distinction however: FSkeletalMeshImportData::ApplyRigToGeo seem to work based on VertexInstances,
	 *										it also checks the VertexCandidate Normal and UVs and only finds the candidate legitimate if they match between Rig and Geo)
	 *										As Influences (BoneIndex and BoneWeights) are Vertex (NOT VertexInstance) dependent.
	 *										Whilst original implementation in FSkeletalMeshImportData::ApplyRigToGeo was checking and validating against normals and UVs for NearestWedges,
	 *										with current implementation we try the NearestVertices with the same principle as the FindMatchingPositionVertexIndexes. (aka based on GetSmallestDeltaBetweenTriangleLists)
	 */
	static SKELETALMESHDESCRIPTION_API void ApplyRigToGeo(FMeshDescription& RigMeshDescription /*Base/From/'Other'*/, FMeshDescription& GeoMeshDescription /*Target/To*/);

	/**
	 * Prune the skin weights on the given profiles to match the bone weight settings.
	 * @param InOutMeshDescription The mesh description to modify.
	 * @param InBoneWeightSettings The bone weight settings to use for controlling the pruning. 
	 * @param InProfileNames The profile names to prune. If empty, will prune every profile. For only the default
	 *   profile, use FSkeletalMeshAttributes::DefaultSkinWeightProfileName
	 */
	static SKELETALMESHDESCRIPTION_API void PruneBoneWeights(
		FMeshDescription& InOutMeshDescription,
		const UE::AnimationCore::FBoneWeightsSettings& InBoneWeightSettings,
		TConstArrayView<FName> InProfileNames = {}
		);

	/**
	 * Splits the mesh into further polygon groups if the number of unique bones being bound to by all the triangles
	 * in that group exceeds the given maximum bone count. If a split happens, a new polygroup attribute is created 
	 * ("ParentPolygonGroup") that points to the polygon group ID that the split-off group got created from.
	 * @param InOutMeshDescription The mesh to partition.
	 * @param InMaxBoneCount The maximum number of unique bones in each group.
	 * @param InProfileNames The skin weight profiles to consider. If empty, will consider every profile. For only the 
	 *   default profile, use FSkeletalMeshAttributes::DefaultSkinWeightProfileName
	 */
	static SKELETALMESHDESCRIPTION_API void PartitionByMaximumUniqueBoneIndexCount(
		FMeshDescription& InOutMeshDescription,
		const int32 InMaxBoneCount,
		TConstArrayView<FName> InProfileNames = {}
		);
	
	/**
	 * Removes the specified bones, and its children, from the mesh on the given skin weight profiles. If the list of profiles
	 * is empty, then the bones are removed from all profiles. Any bones to remove that don't exist on the mesh will be ignored.
	 * All weights associated with removed bones are shifted to that bone's parent.
	 * If the mesh has no bone attributes, this operation is a no-op and the mesh is left unchanged.
	 * @note It is not allowed to remove the root bone.
	 * 
	 * @param InOutMeshDescription The skeletal mesh object to remove bones and bone bindings from.
	 * @param InBoneNames The list of bone names to remove.
	 * @param InBoneWeightSettings The settings to use when setting the new bone weights.
	 * @param InProfileNames The list of skin weight profiles to operate on. If empty, all skin weight profiles will be
	 *   processed.
	 */
	static SKELETALMESHDESCRIPTION_API void RemoveBones(
		FMeshDescription& InOutMeshDescription,
		TConstArrayView<FName> InBoneNames,
		const UE::AnimationCore::FBoneWeightsSettings& InBoneWeightSettings,
		TConstArrayView<FName> InProfileNames = {}
		);

	/**
	 * Removes any unused bones from the bone attribute list and adjust the parent indices accordingly. All profiles are checked
	 * for bones-in-use.
	 * 
	 * @param InOutMeshDescription The skeletal mesh object to remove unused bones from.
	 */
	static SKELETALMESHDESCRIPTION_API void RemoveUnusedBones(
		FMeshDescription& InOutMeshDescription
		);
};
