// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshDescription.h"
#include "BoneWeights.h"


namespace Internal
{
	struct FAttributesRefAdapterConst
	{
		using FBoneWeight = UE::AnimationCore::FBoneWeight;

		using ContainerType = const TArrayAttribute<const int32>;

		static int32 Num(const ContainerType& InContainer)
		{
			return InContainer.Num();
		}

		static FBoneWeight Get(const ContainerType& InContainer, int32 InIndex)
		{
			return FBoneWeight::FromInt32(InContainer.GetData()[InIndex]);
		}

		template<typename Predicate>
        static int32 IndexOf(const ContainerType& InContainer, Predicate InPredicate)
		{
			auto ProxyPredicate = [InPredicate](const int32 InV) -> bool
			{
				return InPredicate(FBoneWeight::FromInt32(InV));
			};
			return InContainer.IndexOfByPredicate(ProxyPredicate);
		}
	};

	struct FAttributesRefAdapter
	{
		using FBoneWeight = UE::AnimationCore::FBoneWeight;

		using ContainerType = TArrayAttribute<int32>;

		static void SetNum(ContainerType& InContainer, int32 InNum)
		{
			InContainer.SetNumUninitialized(InNum);
		}

		static int32 Num(const ContainerType& InContainer)
		{
			return InContainer.Num();
		}

		static FBoneWeight Get(const ContainerType& InContainer, int32 InIndex)
		{
			return FBoneWeight::FromInt32(InContainer.GetData()[InIndex]);
		}

		static void Set(ContainerType& InContainer, int32 InIndex, FBoneWeight InBoneWeight)
		{
			InContainer.GetData()[InIndex] = InBoneWeight.ToInt32();
		}

		static void Add(ContainerType& InContainer, FBoneWeight InBoneWeight)
		{
			InContainer.Add(InBoneWeight.ToInt32());
		}

		static void Remove(ContainerType& InContainer, int32 InIndex)
		{
			InContainer.RemoveAt(InIndex, 1);
		}

		template<typename Predicate>
        static void Sort(ContainerType& InContainer, Predicate InPredicate)
		{
			auto ProxyPredicate = [InPredicate](const int32 InA, const int32 InB) -> bool
			{
				return InPredicate(FBoneWeight::FromInt32(InA), FBoneWeight::FromInt32(InB));
			};
			InContainer.SortByPredicate(ProxyPredicate);
		}

		template<typename Predicate>
        static int32 IndexOf(const ContainerType& InContainer, Predicate InPredicate)
		{
			auto ProxyPredicate = [InPredicate](const int32 InV) -> bool
			{
				return InPredicate(FBoneWeight::FromInt32(InV));
			};
			return InContainer.IndexOfByPredicate(ProxyPredicate);
		}
	};


	template<typename AdaptorType, typename StorageType>
	class TVertexBoneWeightsBase :
	    public UE::AnimationCore::TBoneWeights<AdaptorType>
	{
		using Super = UE::AnimationCore::TBoneWeights<AdaptorType>;
	public:	
		explicit TVertexBoneWeightsBase(const TArrayAttribute<StorageType> InContainer) :
	        Super(Container),
	        Container(InContainer)
		{}

		// For compatibility with range-based for loops.
		class const_iterator
		{
		public:
			const_iterator& operator++()
			{
				++Index;
				return *this;
			}

			bool operator!=(const const_iterator &InOther) const
			{
				return Weights != InOther.Weights || Index != InOther.Index;
			}			

			UE::AnimationCore::FBoneWeight operator*() const
			{
				return Weights->Get(Index);
			}
		
		protected:
			friend class TVertexBoneWeightsBase<AdaptorType, StorageType>;

			const_iterator() = delete;
			
			explicit const_iterator(const TVertexBoneWeightsBase<AdaptorType, StorageType> *InWeights, int32 InIndex) :
	            Weights(InWeights),
	            Index(InIndex)
			{
				check(InWeights);
				check(InIndex > INDEX_NONE && InIndex <= InWeights->Num());
			}

			const TVertexBoneWeightsBase<AdaptorType, StorageType> *Weights = nullptr;
			int32 Index = INDEX_NONE;
		};

		const_iterator begin() const { return const_iterator(this, 0); }		
		const_iterator end() const { return const_iterator(this, this->Num()); }

	private:
		TArrayAttribute<StorageType> Container;
	};

} // namespace Internal

using FVertexBoneWeights = Internal::TVertexBoneWeightsBase<Internal::FAttributesRefAdapter, int32>;
using FVertexBoneWeightsConst = Internal::TVertexBoneWeightsBase<Internal::FAttributesRefAdapterConst, const int32>;


// Merge these two.
class FSkinWeightsVertexAttributesRef
{
public:
	FSkinWeightsVertexAttributesRef() = default;
	FSkinWeightsVertexAttributesRef(const FSkinWeightsVertexAttributesRef &) = default;
	FSkinWeightsVertexAttributesRef(FSkinWeightsVertexAttributesRef &&) noexcept = default;
	FSkinWeightsVertexAttributesRef& operator=(const FSkinWeightsVertexAttributesRef &InAttributesRef)
	{
		AttributesRef = InAttributesRef.AttributesRef;
		return *this;
	}
	
	FSkinWeightsVertexAttributesRef& operator=(FSkinWeightsVertexAttributesRef &&InAttributesRef) noexcept
	{
		AttributesRef = MoveTemp(InAttributesRef.AttributesRef);
		InAttributesRef.AttributesRef = TVertexAttributesRef<TArrayAttribute<int32>>();
		return *this;
	}

	bool IsValid() const
	{
		return AttributesRef.IsValid();
	}

	FVertexBoneWeights Get(const FVertexID InVertexIndex) const
	{
		return FVertexBoneWeights(AttributesRef.Get(InVertexIndex));
	}

	/**
	 * Set the weights using properly normalized weights.
	 * 
	 * @param InVertexIndex The vertex index to set these weights at. 
	 * @param InWeights The normalized weights to set.
	 */
	void Set(const FVertexID InVertexIndex, const UE::AnimationCore::FBoneWeights &InWeights)
	{
		// FBoneWeights guarantees ordering and normalization, so we just copy the raw data.
		TArrayAttribute<int32> Weights = AttributesRef.Get(InVertexIndex);

		Weights.SetNumUninitialized(InWeights.Num());
		for (int32 Index = 0; Index < InWeights.Num(); Index++)
		{
			Weights[Index] = InWeights[Index].ToInt32();
		}
	}

	/**
	* Set bone weights using potentially unnormalized weights. The given weights will be pruned and normalized according
	* to the provided settings. If no settings are provided, default settings will be used.
	* 
	* @param InVertexIndex The vertex index to set these weights at.
	* @param InWeights The raw weights to set.
	* @param InSettings Optional settings for pruning and normalization.
	*/
	void Set(
		const FVertexID InVertexIndex, 
		TArrayView<const UE::AnimationCore::FBoneWeight> InWeights,
		const UE::AnimationCore::FBoneWeightsSettings& InSettings = {}
		)
	{
		// Proxy through FVertexBoneWeights so we get proper sorting and normalization of the weights. 
		FVertexBoneWeights BoneWeights(AttributesRef.Get(InVertexIndex));

		BoneWeights.SetBoneWeights(InWeights, InSettings);
	}
	
	/** 
	 * Blindly set raw weights. The user is responsible to make sure that the weights are normalized. No checking
	 * of normalization, maximum weight count, or minimal weight threshold will be performed. 
	 *
	 * @param InVertexIndex The vertex index to set these weights at.
	 * @param InWeights The raw weights to set.
	 */
	void SetRaw(
		const FVertexID InVertexIndex, 
		TArrayView<const UE::AnimationCore::FBoneWeight> InWeights
		)
	{
		TArray<int32, TInlineAllocator<UE::AnimationCore::MaxInlineBoneWeightCount>> RawWeights;
		RawWeights.Reserve(InWeights.Num());
		
		for (const UE::AnimationCore::FBoneWeight& Weight : InWeights)
		{
			RawWeights.Add(Weight.ToInt32());
		}
		AttributesRef.SetArrayView(InVertexIndex, RawWeights);
	}
	
protected:
	friend class FSkeletalMeshAttributes;

	FSkinWeightsVertexAttributesRef(TVertexAttributesRef<TArrayAttribute<int32>> InAttributesRef)
        : AttributesRef(InAttributesRef)
	{}

private:
	friend class FSkinWeightsVertexAttributesConstRef;
	TVertexAttributesRef<TArrayAttribute<int32>> AttributesRef;
};


class FSkinWeightsVertexAttributesConstRef
{
public:
	FSkinWeightsVertexAttributesConstRef() = default;
	FSkinWeightsVertexAttributesConstRef(const FSkinWeightsVertexAttributesConstRef &) = default;
	FSkinWeightsVertexAttributesConstRef(FSkinWeightsVertexAttributesConstRef &&) noexcept = default;
	FSkinWeightsVertexAttributesConstRef& operator=(const FSkinWeightsVertexAttributesConstRef &InAttributesRef)
	{
		AttributesConstRef = InAttributesRef.AttributesConstRef;
		return *this;
	}
	
	FSkinWeightsVertexAttributesConstRef& operator=(FSkinWeightsVertexAttributesConstRef &&InAttributesRef) noexcept
	{
		AttributesConstRef = MoveTemp(InAttributesRef.AttributesConstRef);
		InAttributesRef.AttributesConstRef = TVertexAttributesConstRef<TArrayAttribute<int32>>();
		return *this;
	}

	// Converting constructors from the non-const variant
	explicit FSkinWeightsVertexAttributesConstRef(const FSkinWeightsVertexAttributesRef &InAttribs) :
		AttributesConstRef(InAttribs.AttributesRef)
	{}
	
	FSkinWeightsVertexAttributesConstRef& operator=(const FSkinWeightsVertexAttributesRef &InAttribs)
	{
		AttributesConstRef = InAttribs.AttributesRef;
		return *this;
	}
	
	bool IsValid() const
	{
		return AttributesConstRef.IsValid();
	}
	
	FVertexBoneWeightsConst Get(FVertexID InVertexIndex) const
	{
		return FVertexBoneWeightsConst(AttributesConstRef.Get(InVertexIndex));
	}

protected:
	friend class FSkeletalMeshAttributes;
	friend class FSkeletalMeshAttributesShared;

	FSkinWeightsVertexAttributesConstRef(TVertexAttributesRef<TArrayAttribute<int32>> InAttributesConstRef)
        : AttributesConstRef(InAttributesConstRef)
	{}

	FSkinWeightsVertexAttributesConstRef(TVertexAttributesRef<TArrayAttribute<const int32>> InAttributesConstRef)
	    : AttributesConstRef(InAttributesConstRef)
	{}

private:
	TVertexAttributesConstRef<TArrayAttribute<int32>> AttributesConstRef;
};
