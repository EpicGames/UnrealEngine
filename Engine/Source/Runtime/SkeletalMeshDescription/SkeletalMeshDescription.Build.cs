// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SkeletalMeshDescription : ModuleRules
	{
		public SkeletalMeshDescription(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"AnimationCore",	// For BoneWeights
					"Engine",			// For GPUSkinPublicDefs.h
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"GeometryCore",
					"MeshConversion"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"MeshDescription",
					"StaticMeshDescription"
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");
		}
	}
}
